{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Netcode Unit Test",
	"Description" : "A unit testing framework for testing the Unreal Engine netcode, primarily for bugs and exploits",
	"Category" : "Networking",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "NetcodeUnitTest",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"Linux"
			]
		}
	]
}

