// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/ObjectPtr.h"

#include "NetcodeUnitTest.h"
#include "ProcessUnitTest.h"
#include "NUTEnum.h"

#include "ClientUnitTest.generated.h"


// Forward declarations
class AActor;
class ANUTActor;
class UMinimalClient;
class AOnlineBeaconClient;
class APlayerController;
class FInBunch;
class FFuncReflection;
class UActorChannel;
class UChannel;
class UNetConnection;
class UNetDriver;
struct FFrame;
struct FOutParmRec;
struct FUniqueNetIdRepl;
enum class ENUTControlCommand : uint8;



/**
 * Base class for all unit tests depending upon a MinimalClient connecting to a server.
 * The MinimalClient handles creation/cleanup of an entire new UWorld, UNetDriver and UNetConnection, for fast unit testing.
 *
 * NOTE: See NUTEnum.h, for important flags for configuring unit tests and the minimal client.
 * 
 * In subclasses, implement the unit test within the ExecuteClientUnitTest function (remembering to call parent)
 */
UCLASS(Abstract)
class NETCODEUNITTEST_API UClientUnitTest : public UProcessUnitTest
{
	GENERATED_UCLASS_BODY()

	friend class FScopedLog;
	friend class FScopedNetObjectReplace;
	friend class UUnitTestActorChannel;
	friend class UUnitTestManager;
	friend class FUnitTestEnvironment;
	friend class UMinimalClient;


	/** Variables which should be specified by every subclass (some depending upon flags) */
protected:
	/** All of the internal unit test parameters/flags, for controlling state and execution */
	EUnitTestFlags UnitTestFlags;

	/** Flags for configuring the minimal client - lots of interdependencies between these and UnitTestFlags */
	EMinClientFlags MinClientFlags;

	/** The class to use for the MinimalClient */
	TSubclassOf<UMinimalClient> MinClientClass;


	/** The base URL the server should start with */
	FString BaseServerURL;

	/** The (non-URL) commandline parameters the server should be launched with */
	FString BaseServerParameters;

	// @todo #JohnB: There is duplication between this and MinimalClient - deprecate this from here?
	/** If connecting to a beacon, the beacon type name we are connecting to */
	FString ServerBeaconType;

	/** The base URL clients should start with */
	FString BaseClientURL;

	/** The (non-URL) commandline parameters clients should be launched with */
	FString BaseClientParameters;

	/** Actors the server is allowed replicate to client (requires AllowActors flag). Use NotifyAllowNetActor for conditional allows. */
	UPROPERTY()
	TArray<TObjectPtr<UClass>> AllowedClientActors;

	/** Clientside RPC's that should be allowed to execute (requires minimal client NotifyProcessNetEvent flag) */
	TArray<FString> AllowedClientRPCs;

	/** Runtime variables */
protected:
	/** Reference to the created server process handling struct */
	TWeakPtr<FUnitTestProcess> ServerHandle;

	/** The address of the launched server */
	FString ServerAddress;

	/** The address of the server beacon (if UnitTestFlags are set to connect to a beacon) */
	FString BeaconAddress;

	/** Reference to the created client process handling struct (if enabled) */
	TWeakPtr<FUnitTestProcess> ClientHandle;

	/** Whether or not there is a blocking event/process preventing setup of the server */
	bool bBlockingServerDelay;

	/** Whether or not there is a blocking event/process preventing setup of a client */
	bool bBlockingClientDelay;

	/** Whether or not there is a blocking event/process preventing the minimal client from connecting */
	bool bBlockingMinClientDelay;

	/** When a server is launched after a blocking event/process, this delays the launch of any clients, in case of more blockages */
	double NextBlockingTimeout;


	/** The object which handles implementation of the minimal client */
	UPROPERTY()
	TObjectPtr<UMinimalClient> MinClient;

	/** Whether or not the initial connect of the minimal client was triggered */
	bool bTriggerredInitialConnect;


	/** Client state variables */
protected:
	/** Stores a reference to the replicated PlayerController (if set to wait for this), after NotifyHandleClientPlayer */
	UPROPERTY()
	TWeakObjectPtr<APlayerController> UnitPC;

	/** Whether or not the UnitPC Pawn was fully setup (requires EUnitTestFlags::RequirePawn) */
	bool bUnitPawnSetup;

	/** Whether or not the UnitPC PlayerState was fully setup (requires EUnitTestFlags::RequirePlayerState) */
	bool bUnitPlayerStateSetup;

	/** If EUnitTestFlags::RequireNUTActor is set, stores a reference to the replicated NUTActor */
	UPROPERTY()
	TWeakObjectPtr<ANUTActor> UnitNUTActor;

	/** Whether or not UnitNUTActor is fully setup, i.e. has replicated its Owner */
	bool bUnitNUTActorSetup;

	/** If EUnitTestFlags::RequireBeacon is set, stores a reference to the replicated beacon */
	UPROPERTY()
	TWeakObjectPtr<AActor> UnitBeacon;

	/** If EUnitTestFlags::RequirePing is true, whether or not we have already received the pong */
	bool bReceivedPong;

	/** An expected network failure occurred, which will be handled during the next tick instead of immediately */
	bool bPendingNetworkFailure;


private:
	/** Cached reference to the OnlineBeaconClient static class */
	UPROPERTY()
	TObjectPtr<UClass> OnlineBeaconClass_Private;


	void InitOnlineBeaconClass();

	UClass* GetOnlineBeaconClass()
	{
		if (!OnlineBeaconClass_Private)
		{
			InitOnlineBeaconClass();
		}

		return OnlineBeaconClass_Private;
	}


	/**
	 * Interface and hooked events for client unit tests
	 */
public:
	/**
	 * Override this, to implement the client unit test
	 * NOTE: Should be called last, in overridden functions
	 * IMPORTANT: EndUnitTest should be triggered, upon completion of the unit test (which may be delayed, for many unit tests)
	 */
	virtual void ExecuteClientUnitTest() PURE_VIRTUAL(UClientUnitTest::ExecuteClientUnitTest,);


	/**
	 * Gives subclass UnitTest's an opportunity to alter the MinimalClient setup parameters
	 *
	 * @param Parms		Reference to the MinimalClient parameters prior to setup/connection
	 */
	virtual void NotifyAlterMinClient(FMinClientParms& Parms);

	/**
	 * Notification from the minimal client, that it has fully connected
	 */
	virtual void NotifyMinClientConnected();

	/**
	 * Override this, to receive notification of NMT_NUTControl messages, from the server
	 *
	 * @param CmdType		The command type
	 * @param Command		The command being received
	 */
	virtual void NotifyNUTControl(ENUTControlCommand CmdType, FString Command)
	{
	}

	/**
	 * Override this, to receive notification of all other non-NMT_NUTControl control messages
	 *
	 * @param Bunch			The bunch containing the control message
	 * @param MessageType	The control message type
	 */
	virtual void NotifyControlMessage(FInBunch& Bunch, uint8 MessageType);

	/**
	 * Notification that the local net connections PlayerController has been replicated and is being setup
	 *
	 * @param PC			The new local PlayerController
	 * @param Connection	The connection associated with the PlayerController
	 */
	virtual void NotifyHandleClientPlayer(APlayerController* PC, UNetConnection* Connection);

	/**
	 * Notification triggered BEFORE a replicated actor has been created (allowing you to block creation, based on class)
	 *
	 * @param ActorClass	The actor class that is about to be created
	 * @param bActorChannel	Whether or not this actor is being created within an actor channel
	 * @param bBlockActor	Whether or not to block creation of that actor (defaults to true)
	 */
	virtual void NotifyAllowNetActor(UClass* ActorClass, bool bActorChannel, bool& bBlockActor);

	/**
	 * Override this, to receive notification AFTER an actor channel actor has been created
	 *
	 * @param ActorChannel	The actor channel associated with the actor
	 * @param Actor			The actor that has just been created
	 */
	virtual void NotifyNetActor(UActorChannel* ActorChannel, AActor* Actor);

	/**
	 * Triggered upon a network connection failure
	 *
	 * @param FailureType	The reason for the net failure
	 * @param ErrorString	More detailed error information
	 */
	virtual void NotifyNetworkFailure(ENetworkFailure::Type FailureType, const FString& ErrorString);


	/**
	 * If EUnitTestFlags::CaptureReceiveRaw is set, this is triggered for every packet received from the server
	 * NOTE: Data is a uint8 array, of size 'NETWORK_MAX_PACKET', and elements can safely be modified
	 *
	 * @param Data		The raw data/packet being received (this data can safely be modified, up to length 'NETWORK_MAX_PACKET')
	 * @param Count		The amount of data received (if 'Data' is modified, this should be modified to reflect the new length)
	 */
	virtual void NotifyReceivedRawPacket(void* Data, int32& Count)
	{
	}

	/**
	 * Triggered for every packet sent to the server, when LowLevelSend is called.
	 * IMPORTANT: This occurs AFTER PacketHandler's have had a chance to modify packet data
	 *
	 * NOTE: Don't consider data safe to modify (will need to modify the implementation, if that is desired)
	 *
	 * @param Data			The raw data/packet being sent
	 * @param Count			The amount of data being sent
	 * @param bBlockSend	Whether or not to block the send (defaults to false)
	 */
	virtual void NotifySocketSendRawPacket(void* Data, int32 Count, bool& bBlockSend)
	{
	}

	/**
	 * Bunches received on the control channel. These need to be parsed manually,
	 * because the control channel is intentionally disrupted (otherwise it can affect the main client state)
	 *
	 * @param Bunch			The bunch being received
	 */
	virtual void ReceivedControlBunch(FInBunch& Bunch);


	/**
	 * Overridable in subclasses - can be used to control/block any script events, other than receiving of RPC's (see NotifyReceiveRPC)
	 *
	 * @param Actor			The actor the event is being executed on
	 * @param Function		The script function being executed
	 * @param Parameters	The raw unparsed parameters, being passed into the function
	 * @param bBlockEvent	Whether or not to block the event from executing
	 */
	virtual void NotifyProcessEvent(AActor* Actor, UFunction* Function, void* Parameters, bool& bBlockEvent)
	{
	}

	/**
	 * Overridable in subclasses - can be used to control/block receiving of RPC's
	 *
	 * @param Actor			The actor the RPC is being executed on
	 * @param Function		The RPC being executed
	 * @param Parameters	The raw unparsed parameters, being passed into the function
	 * @param bBlockRPC		Whether or not to block the RPC from executing
	 */
	virtual void NotifyReceiveRPC(AActor* Actor, UFunction* Function, void* Parameters, bool& bBlockRPC);

	/**
	 * Overridable in subclasses - can be used to control/block sending of RPC's
	 *
	 * @param Actor				The actor the RPC will be called in
	 * @param Function			The RPC to call
	 * @param Parameters		The parameters data blob
	 * @param OutParms			Out parameter information (irrelevant for RPC's)
	 * @param Stack				The script stack
	 * @param SubObject			The sub-object the RPC is being called in (if applicable)
	 * @param bBlockSendRPC		Whether or not to allow sending of the RPC
	 */
	virtual void NotifySendRPC(AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack,
								UObject* SubObject, bool& bBlockSendRPC)
	{
	}


	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;

	virtual void NotifyProcessFinished(TWeakPtr<FUnitTestProcess> InProcess) override;

	virtual void NotifyProcessSuspendState(TWeakPtr<FUnitTestProcess> InProcess, ESuspendState InSuspendState) override;


	virtual void NotifySuspendRequest() override;

	virtual bool NotifyConsoleCommandRequest(FString CommandContext, FString Command) override;

	virtual void GetCommandContextList(TArray<TSharedPtr<FString>>& OutList, FString& OutDefaultContext) override;


	/**
	 * Utility functions for use by subclasses
	 */
public:
	/**
	 * Sends an NMT_NUTControl control channel message, for the server NUTActor. See NUTActor.h
	 *
	 * @param CommandType	The type of NUTActor control channel command being sent.
	 * @param Command		The command parameters.
	 * @return				Whether or not the command was sent successfully.
	 */
	bool SendNUTControl(ENUTControlCommand CommandType, FString Command);

	/**
	 * See UMinimalClient
	 */
	bool SendRPCChecked(UObject* Target, const TCHAR* FunctionName, void* Parms, int16 ParmsSize, int16 ParmsSizeCorrection=0);

	/**
	 * See UMinimalClient
	 */
	bool SendRPCChecked(UObject* Target, FFuncReflection& FuncRefl);


	/**
	 * As above, except executes a static UFunction in the unit test (must be prefixed with UnitTestServer_), on the unit test server,
	 * allowing unit tests to define and contain their own 'pseudo'-RPC's.
	 *
	 * Functions that you want to call, must match this function template:
	 *	UFUNCTION()
	 *	static void UnitTestServer_Func(ANUTActor* InNUTActor);
	 *
	 * @param RPCName	The name of the pseudo-RPC, which should be executed
	 * @return			Whether or not the pseudo-RPC was sent successfully
	 */
	FORCEINLINE bool SendUnitRPCChecked(FString RPCName)
	{
		return SendUnitRPCChecked_Internal(this, RPCName);
	}

	/**
	 * As above, except allows 'UnitTestServer' RPC's to be located in an arbitrary class (e.g. if shared between unit tests),
	 * specified as the delegate parameter.
	 *
	 * @param TargetClass	The class which the 'UnitTestServer' function resides in.
	 * @param RPCName		The name of the pseudo-RPC, which should be executed
	 * @return				Whether or not the pseudo-RPC was sent successfully
	 */
	template<class TargetClass>
	FORCEINLINE bool SendUnitRPCChecked(FString RPCName)
	{
		return SendUnitRPCChecked_Internal(GetMutableDefault<TargetClass>(), RPCName);
	}

private:
	/**
	 * Internal implementation of the above functions
	 */
	bool SendUnitRPCChecked_Internal(UObject* Target, FString RPCName);

public:
	/**
	 * Callback for when an RPC send fails on the minimal client
	 */
	void OnRPCFailure();


	/**
	 * Gets the generic log message that is used to indicate unit test failure
	 */
	static FORCEINLINE const TCHAR* GetGenericExploitFailLog()
	{
		return TEXT("Blank exploit fail log message");
	}

	/**
	 * Sends a generic log message to the server, which (if successfully logged) indicates unit test failure.
	 * This is for use with unit tests that are expecting a crash.
	 */
	void SendGenericExploitFailLog();

	/**
	 * Internal base implementation and utility functions for client unit tests
	 */
protected:
	/**
	 * Validates, both at compile time (template params) or at runtime (function params), that the specified flags are valid.
	 *
	 * When specifying the whole flag list in one go, do a compile time check using the template parameters.
	 * When modifying a runtime-written flag list, do a runtime check on the final flag variables using the function parameters.
	 *
	 * @param CompileTimeUnitFlags	The compile-time unit test flag list to be checked
	 * @param CompileTimeMinFlags	The compile-time minimal client flag list to be checked
	 * @param RuntimeUnitFlags		The runtime unit test flag list to be checked
	 * @param RuntimeMinFlags		The runtime minimal client flag list to be checked
	 */
	template<EUnitTestFlags CompileTimeUnitFlags=EUnitTestFlags::None, EMinClientFlags CompileTimeMinFlags=EMinClientFlags::None>
	void ValidateUnitFlags(EUnitTestFlags RuntimeUnitFlags=EUnitTestFlags::None, EMinClientFlags RuntimeMinFlags=EMinClientFlags::None)
	{
		// Validate EMinClientFlags
		ValidateMinFlags<CompileTimeMinFlags>(RuntimeMinFlags);

		PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS

		// Implements the '*Flags' variables within 'Condition', for both static/compile-time checks, and runtime checks
		#define FLAG_ASSERT(Condition, Message) \
			{ \
				struct CompileTimeAssert \
				{ \
					void NeverCalled() \
					{ \
						constexpr EUnitTestFlags UnitTestFlags = CompileTimeUnitFlags; \
						constexpr EMinClientFlags MinClientFlags = CompileTimeMinFlags; \
						static_assert((CompileTimeUnitFlags == EUnitTestFlags::None && CompileTimeMinFlags == EMinClientFlags::None) \
										|| (Condition), Message); \
					} \
				}; \
				\
				struct RuntimeAssert \
				{ \
					static void DoCheck(EUnitTestFlags UnitTestFlags, EMinClientFlags MinClientFlags) \
					{ \
						UNIT_ASSERT(Condition); \
					} \
				}; \
				\
				if (RuntimeUnitFlags != EUnitTestFlags::None || RuntimeMinFlags != EMinClientFlags::None) \
				{ \
					RuntimeAssert::DoCheck(RuntimeUnitFlags, RuntimeMinFlags); \
				} \
			}



		FLAG_ASSERT(!!(UnitTestFlags & EUnitTestFlags::LaunchServer),
					"Currently, unit tests don't support NOT launching/connecting to a server");

		FLAG_ASSERT((!(UnitTestFlags & EUnitTestFlags::AcceptPlayerController) && !(UnitTestFlags & EUnitTestFlags::RequireNUTActor)) ||
						!!(MinClientFlags & EMinClientFlags::AcceptActors),
					"If you require a player/NUTActor, you need to accept actor channels");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequireNUTActor) || !!(UnitTestFlags & EUnitTestFlags::AcceptPlayerController) ||
					!!(UnitTestFlags & EUnitTestFlags::RequireBeacon),
					"If you require a NUTActor, you need to either accept a PlayerController or require a beacon");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePlayerController) ||
						!!(UnitTestFlags & EUnitTestFlags::AcceptPlayerController),
					"Don't require a PlayerController, if you don't accept one");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePawn) || !!(UnitTestFlags & EUnitTestFlags::RequirePlayerController),
					"If you require a pawn, you must require a PlayerController");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePawn) || !!(MinClientFlags & EMinClientFlags::NotifyProcessNetEvent),
					"If you require a pawn, you must enable NotifyProcessNetEvent");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePlayerState) ||
					!!(UnitTestFlags & EUnitTestFlags::RequirePlayerController),
					"If you require a PlayerState, you must require a PlayerController");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePawn) || !!(MinClientFlags & EMinClientFlags::NotifyNetActors),
					"For part of pawn-setup detection, you need notification for net actors");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePlayerState) || !!(MinClientFlags & EMinClientFlags::NotifyNetActors),
					"For part of PlayerState-setup detection, you need notification for net actors");

		// As above, but with the 'custom' requirements flag
		// NOTE: Removed, as it can still be useful to have other requirements flags
		//FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequireCustom) ||
		//				FMath::IsPowerOfTwo((uint32)(UnitTestFlags & EUnitTestFlags::RequirementsMask)));

		FLAG_ASSERT(!(MinClientFlags & EMinClientFlags::SendRPCs) || !!(UnitTestFlags & EUnitTestFlags::AcceptPlayerController) ||
					!!(UnitTestFlags & EUnitTestFlags::BeaconConnect),
					"You can't send RPC's, without accepting a player controller (netcode blocks this, without a PC); "
					"unless this is a beacon");

		// If connecting to a beacon, a number of unit test flags are not supported
		const EUnitTestFlags RejectedBeaconFlags = (EUnitTestFlags::AcceptPlayerController | EUnitTestFlags::RequirePlayerController |
													EUnitTestFlags::RequirePing);

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::BeaconConnect) || !(UnitTestFlags & RejectedBeaconFlags),
					"Some unit test flags are incompatible with EUnitTestFlags::BeaconConnect");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::BeaconConnect) || !!(MinClientFlags & EMinClientFlags::NotifyNetActors),
					"If connecting to a beacon, net actor notification is required, for proper setup");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequireBeacon) || !!(UnitTestFlags & EUnitTestFlags::BeaconConnect),
					"Don't require a beacon, if you're not connecting to a beacon");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::LaunchClient) || !!(UnitTestFlags & EUnitTestFlags::LaunchServer),
					"Don't specify server-dependent flags, if not auto-launching a server");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequireNUTActor) || !!(MinClientFlags & EMinClientFlags::NotifyNetActors),
					"You can't use 'RequireNUTActor', without net actor notifications");

		FLAG_ASSERT(!(UnitTestFlags & EUnitTestFlags::ExpectServerCrash) || !!(UnitTestFlags & EUnitTestFlags::ExpectDisconnect),
					"If a unit test expects a server crash, it should also expect a disconnect too "
					"(to avoid an invalid 'unit test needs update' result)");

		#undef FLAG_ASSERT

		PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS 
	}

	/**
	 * Sets and validates at compile time, that the specified flags are valid.
	 * NOTE: If your unit test subclasses another, it will have to manually change UnitTestFlags/MinClientFlags at runtime.
	 *
	 * @param CompileTimeUnitFlags	The compile-time flag list to be checked
	 * @param CompileTimeMinFlags	The compile-time flag list to be checked
	 */
	template<EUnitTestFlags CompileTimeUnitFlags=EUnitTestFlags::None, EMinClientFlags CompileTimeMinFlags=EMinClientFlags::None>
	void SetFlags()
	{
		ValidateUnitFlags<CompileTimeUnitFlags, CompileTimeMinFlags>();

		UnitTestFlags = CompileTimeUnitFlags;
		MinClientFlags = CompileTimeMinFlags;
	}

	virtual bool ValidateUnitTestSettings(bool bCDOCheck=false) override;

	virtual void ResetTimeout(FString ResetReason, bool bResetConnTimeout=false, uint32 MinDuration=0) override;


	/**
	 * Returns the requirements flags, that this unit test currently meets
	 */
	EUnitTestFlags GetMetRequirements();

	/**
	 * Whether or not all 'requirements' flag conditions have been met
	 *
	 * @param bIgnoreCustom		If true, checks all requirements other than custom requirements
	 * @return					Whether or not all requirements are met
	 */
	bool HasAllRequirements(bool bIgnoreCustom=false);

public:
	/**
	 * Optionally, if the 'RequireCustom' flag is set, this returns whether custom conditions have been met.
	 *
	 * Only use this if you are mixing multiple 'requirements' flags - if just using 'RequireCustom',
	 * trigger 'ExecuteClientUnitTest' manually
	 *
	 * NOTE: You still have to check 'HasAllRequirements' and trigger 'ExecuteClientUnitTest', at the time the custom conditions are set
	 */
	virtual bool HasAllCustomRequirements()
	{
		return false;
	}

	/**
	 * Returns the type of log entries that this unit expects to output, for setting up log window filters
	 * (only needs to return values which affect what tabs are shown)
	 *
	 * @return		The log type mask, representing the type of log entries this unit test expects to output
	 */
	virtual ELogType GetExpectedLogTypes() override;

protected:
	virtual bool ExecuteUnitTest() override;

	virtual void CleanupUnitTest(EUnitTestResetStage ResetStage) override;


	/**
	 * Connects a minimal client, to the launched/launching server
	 *
	 * @param InNetID		The unique net id the player should use
	 * @return				Whether or not the connection kicked off successfully
	 */
	virtual bool ConnectMinimalClient(const TCHAR* InNetID=nullptr);

	/**
	 * Cleans up the minimal client
	 */
	virtual void CleanupMinimalClient();

	/**
	 * Triggers an auto-reconnect (disconnect/reconnect) of the minimal client
	 */
	void TriggerAutoReconnect();


	/**
	 * Starts the server process for a particular unit test
	 */
	virtual void StartUnitTestServer();

	/**
	 * Puts together the commandline parameters the server should use, based upon the unit test settings
	 *
	 * @return			The final server commandline
	 */
	virtual FString ConstructServerParameters();

public:
	/**
	 * Determine the next unique server ports to use - incremented to a new unique number, internally
	 *
	 * @param OutServerPort		Outputs the next server port
	 * @param OutBeaconPort		Outputs the next beacon port
	 * @param bAdvance			Whether to advance the internal unique counter, for ports (to e.g. get ports for unlaunched unit tests)
	 */
	static void GetNextServerPorts(int32& OutServerPort, int32& OutBeaconPort, bool bAdvance=true);

protected:
	/**
	 * Starts a client process tied to the unit test, and connects to the specified server address
	 *
	 * @param ConnectIP		The IP address the client should connect to
	 * @param bMinimized	Starts the client with the window minimized (some unit tests need this off)
	 * @return				Returns a pointer to the client process handling struct
	 */
	virtual TWeakPtr<FUnitTestProcess> StartUnitTestClient(FString ConnectIP, bool bMinimized=true);

	/**
	 * Puts together the commandline parameters clients should use, based upon the unit test settings
	 *
	 * @param ConnectIP	The IP address the client will be connecting to
	 * @return			The final client commandline
	 */
	virtual FString ConstructClientParameters(FString ConnectIP);


	virtual void PrintUnitTestProcessErrors(TSharedPtr<FUnitTestProcess> InHandle) override;


	virtual void UnitTick(float DeltaTime) override;

	virtual bool IsTickable() const override;

	virtual void LogComplete() override;

	virtual void UnblockEvents(EUnitTaskFlags ReadyEvents) override;

	virtual bool IsConnectionLogSource(UNetConnection* InConnection) override;


public:
	/**
	 * Accessor for UnitTestFlags
	 *
	 * @return	Returns the value of UnitTestFlags
	 */
	FORCEINLINE EUnitTestFlags GetUnitTestFlags()
	{
		return UnitTestFlags;
	}

	/**
	 * Accessor for ServerHandle
	 *
	 * @return	Returns the value of ServerHandle
	 */
	FORCEINLINE TWeakPtr<FUnitTestProcess> GetServerHandle()
	{
		return ServerHandle;
	}

	/**
	 * Accessor for BeaconAddress
	 */
	FORCEINLINE FString GetBeaconAddress() const
	{
		return BeaconAddress;
	}
};
