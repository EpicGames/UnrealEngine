// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "Misc/CoreMisc.h"
#include "GameFramework/Actor.h"
#include "NetcodeUnitTest.h"
#include "UObject/TextProperty.h"
#include "Net/DataChannel.h"

#include "NUTActor.generated.h"

// Forward declarations
class ANUTActor;

/**
 * Enum for defining custom NetcodeUnitTest control channel commands (sent through NMT_NUTControl)
 */
// @todo #JohnBDoc: Fully document individually
enum class ENUTControlCommand : uint8
{
	Command_NoResult		= 0,
	Command_SendResult		= 1,
	CommandResult_Failed	= 2,
	CommandResult_Success	= 3,
	Ping					= 4,
	Pong					= 5,
	WatchEvent				= 6,
	NotifyEvent				= 7,
	Summon					= 8,
	SuspendProcess			= 9
};

// Custom control channel message, used to communicate with the server NUTActor over the control channel
DEFINE_CONTROL_CHANNEL_MESSAGE(NUTControl, 250, ENUTControlCommand, FString);


/**
 * Delegate for executing a unit test function on the server
 *
 * @param InNUTActor	The serverside NUTActor triggering the delegate
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FExecuteOnServer, ANUTActor*, InNUTActor);


UCLASS()
class ANUTActor : public AActor, public FSelfRegisteringExec
{
	GENERATED_UCLASS_BODY()

private:
	/** The name of the beacon net driver */
	FName BeaconDriverName;

public:
	/** The value of World.RealTimeSeconds as of the last time the client was marked as still alive */
	double LastAliveTime;

	/** A delegate property, used solely for converting strings to delegates */
	UPROPERTY()
	FExecuteOnServer TempDelegate;

	/** Monitors for the creation of the beacon net driver, if -BeaconPort=x was specified on the commandline */
	bool bMonitorForBeacon;


	virtual void PostActorCreated() override;

	virtual UNetConnection* GetNetConnection() const override;

	bool NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch);

	static void NotifyPostLoadMap(UWorld* LoadedWorld);

	static bool VerifyEventWatcher();

	/**
	 * Hooks control channel messages for the specified net driver
	 */
	void HookNetDriver(UNetDriver* TargetNetDriver);


	/**
	 * FExec interface
	 */

	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;


	/**
	 * Executes a console command on the server
	 *
	 * @param Command	The command to be executed
	 */
	UFUNCTION(exec)
	void Admin(FString Command);

	UFUNCTION(reliable, server, WithValidation)
	void ServerAdmin(const FString& Command);


	/**
	 * Triggers seamless travel
	 *
	 * @param Dest	The travel destination
	 */
	UFUNCTION(exec)
	void UnitSeamlessTravel(FString Dest=TEXT(" "));

	/**
	 * Triggers normal travel
	 *
	 * @param Dest	The travel destination
	 */
	UFUNCTION(exec)
	void UnitTravel(FString Dest=TEXT(" "));

	/**
	 * Flushes all pending net connection packets
	 */
	UFUNCTION(exec)
	void NetFlush();

	/**
	 * Makes the game thread wait for the specified number of seconds
	 */
	UFUNCTION(exec)
	void Wait(uint16 Seconds);


	/**
	 * Notifies the server that the client is still around
	 */
	UFUNCTION(reliable, server, WithValidation)
	void ServerClientStillAlive();

	/**
	 * Test log function
	 *
	 * @param InText	The FText passed from the client
	 */
	UFUNCTION(reliable, server, WithValidation)
	NETCODEUNITTEST_API void ServerReceiveText(const FText& InText);


	/**
	 * Send a 'ping' RPC to all clients, to make them log a ping, which unit tests then use to verify the presence of a client process
	 */
	UFUNCTION(reliable, server, WithValidation)
	NETCODEUNITTEST_API void ServerClientPing();


	/**
	 * Received by all clients, emits a ping to log
	 */
	// @todo #JohnBRefactor: When the VM reflection helper is finished, remove NETCODEUNITTEST_API from this, and use reflection instead
	UFUNCTION(reliable, NetMulticast)
	NETCODEUNITTEST_API void NetMulticastPing();


	/**
	 * Executes the function specified in the delegate, on the server (used within unit tests)
	 *
	 * @param InTargetObj	The object the target function is in (automatically resolve to the class default object)
	 * @param InTargetFunc	The function to be executed
	 */
	void NETCODEUNITTEST_API ExecuteOnServer(UObject* InTargetObj, FString InTargetFunc);

	/**
	 * Takes a string representing a delegate (can't replicate delegate parameters), and executes that delegate on the server
	 *
	 * @param InDelegate	A string representing the delegate to be executed
	 */
	UFUNCTION(reliable, server, WithValidation)
	void ServerExecute(const FString& InDelegate);


	virtual void Tick(float DeltaSeconds) override;

	/**
	 * Update the owner for the NUTActor, if the current owner is no longer valid
	 */
	void UpdateOwner();
};


