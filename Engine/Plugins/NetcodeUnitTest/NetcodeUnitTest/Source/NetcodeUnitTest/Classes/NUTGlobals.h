// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/ObjectPtr.h"

#include "NUTGlobals.generated.h"


// Forward declarations
class UNetConnection;
class FUnitTestEnvironment;


/**
 * Stores globals/static-variables for NetcodeUnitTest - for compatibility with hot reload
 */
UCLASS()
class UNUTGlobals : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Singleton
	 */
	FORCEINLINE static UNUTGlobals& Get()
	{
		const TCHAR* GlobalsInstName = TEXT("NUTGlobals_Instance");
		static UNUTGlobals* GlobalsInst = FindObject<UNUTGlobals>(GetTransientPackage(), GlobalsInstName);

		if (GlobalsInst == nullptr)
		{
			GlobalsInst = NewObject<UNUTGlobals>(GetTransientPackage(), FName(GlobalsInstName));

			GlobalsInst->AddToRoot();
		}

		return *GlobalsInst;
	}


public:
	/** For NUTActor - the UNetConnection that 'watch' events will be sent to */
	UPROPERTY()
	TObjectPtr<UNetConnection> EventWatcher;

	/** For ClientUnitTest - the (+10) incremented port number, for each server instance */
	UPROPERTY()
	int32 ServerPortOffset;

	/** For MinimalClient - counter for the number of unit test net drivers created */
	UPROPERTY()
	int32 UnitTestNetDriverCount;

	/** For the '-DumpRPCs' commandline parameter - limits DumpRPCs to RPC's (partially) matching the specified names */
	UPROPERTY()
	TArray<FString> DumpRPCMatches;

	/** List of modules recognized as containing unit tests, plus implementing FNUTModuleInterface for hot reload */
	UPROPERTY()
	TArray<FString> UnitTestModules;

	/** List of unit test modules that have been unloaded, prior to reloading for Hot Reload */
	UPROPERTY()
	TArray<FString> UnloadedModules;
};
