// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "Engine/ActorChannel.h"
#include "UObject/ObjectPtr.h"

#include "UnitTestActorChannel.generated.h"


// Forward declarations
class FInBunch;
class UMinimalClient;


/**
 * An actor net channel override, for hooking ReceivedBunch, to aid in detecting/blocking of remote actors, of a specific class
 */
UCLASS(transient)
class UUnitTestActorChannel : public UActorChannel
{
	friend UMinimalClient;

	GENERATED_UCLASS_BODY()


public:
	virtual void ReceivedBunch(FInBunch& Bunch) override;

	virtual void Tick() override;

	virtual void NotifyActorChannelOpen(AActor* InActor, FInBunch& InBunch) override;

private:
	/** Cached referenced to the minimal client that owns this actor channel */
	UPROPERTY()
	TObjectPtr<UMinimalClient> MinClient;
};

