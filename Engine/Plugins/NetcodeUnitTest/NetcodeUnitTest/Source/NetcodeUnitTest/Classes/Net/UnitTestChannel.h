// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "Engine/Channel.h"
#include "UObject/ObjectPtr.h"

#include "UnitTestChannel.generated.h"


// Forward declarations
class FInBunch;
class UNetConnection;
class UMinimalClient;

/**
 * A net channel for overriding the implementation of traditional net channels,
 * for e.g. blocking control channel messages, to enable minimal clients
 */
UCLASS(transient)
class UUnitTestChannel : public UChannel
{
	GENERATED_UCLASS_BODY()	

	virtual void Init(UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags) override;

	virtual void ReceivedBunch(FInBunch& Bunch) override;

	virtual void Tick() override;


public:
	/** The minimal client which may require received bunch notifications */
	UPROPERTY()
	TObjectPtr<UMinimalClient> MinClient;

	/** Whether or not this channel should verify it has been opened (resends initial packets until acked, like control channel) */
	bool bVerifyOpen;
};




