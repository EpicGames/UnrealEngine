// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "Engine/PackageMapClient.h"
#include "NetcodeUnitTest.h"
#include "UObject/ObjectPtr.h"

#include "UnitTestPackageMap.generated.h"


// Forward declarations
class FArchive;
class UNetConnection;
class UMinimalClient;


// Delegates

/**
 * Delegate for hooking SerializeName
 *
 * @param bPreSerialize		Whether or not this call is occurring before or after serialization (it happens twice)
 * @param bSerializedName	Whether or not the name has already been serialized
 * @param Ar				The archive to serialize from/to
 * @param InName			The name to serialize to/from
 */
DECLARE_MULTICAST_DELEGATE_FourParams(FOnSerializeName, bool /*bPreSerialize*/, bool& /* bSerializedName */, FArchive& /* Ar */, FName& /* InName */);


/**
 * Package map override, for blocking the creation of actor channels for specific actors (by detecting the actor class being created)
 */
UCLASS(transient)
class UUnitTestPackageMap : public UPackageMapClient
{
	GENERATED_UCLASS_BODY()

	virtual bool SerializeObject(FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID* OutNetGUID=nullptr) override;

	virtual bool SerializeName(FArchive& Ar, FName& InName) override;

	virtual bool SerializeNewActor(FArchive& Ar, class UActorChannel* Channel, class AActor*& Actor) override;

public:
	/** Cached reference to the minimal client that owns this package map */
	UPROPERTY()
	TObjectPtr<UMinimalClient> MinClient;

	/** Whether or not we are currently within execution of SerializeNewActor */
	bool bWithinSerializeNewActor;

	/** Whether or not SerializeNewActor is about to spawn an actor, from an archetype */
	bool bPendingArchetypeSpawn;

	/** Map of objects to watch and replace, in SerializeObject */
	TMap<UObject*, UObject*> ReplaceObjects;

	/** Delegate for hooking SerializeName */
	FOnSerializeName OnSerializeName;
};

