// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "Styling/SlateColor.h"
#include "NetcodeUnitTest.h"
#include "HAL/PlatformProcess.h"
#include "UnitTest.h"

#include "ProcessUnitTest.generated.h"

// Enum definitions

/**
 * Enum for different stages of error log parsing (mainly applicable to Unreal Engine processes)
 */
enum class EErrorLogStage : uint8
{
	ELS_NoError,		// No error logs have been received/parsed yet
	ELS_ErrorStart,		// The text indicating the start of an error log is being parsed
	ELS_ErrorDesc,		// The text describing the error is being parsed
	ELS_ErrorCallstack,	// The callstack for the error is being parsed
	ELS_ErrorExit		// The post-error exit message is being parsed (error parsing is effectively complete)
};

/**
 * Enum for specifying the suspend state of a process (typically the server)
 */
enum class ESuspendState : uint8
{
	Active,				// Process is currently active/not-suspended
	Suspended,			// Process is currently suspended
};


// Struct definitions

/**
 * Struct used for storing and classifying each log error line
 */
struct FErrorLog
{
	/** The stage of this error log line */
	EErrorLogStage Stage;

	/** The error log line */
	FString Line;


	/**
	 * Base constructor
	 */
	FErrorLog()
		: Stage(EErrorLogStage::ELS_NoError)
		, Line()
	{
	}

	FErrorLog(EErrorLogStage InStage, const FString& InLine)
		: Stage(InStage)
		, Line(InLine)
	{
	}
};

/**
 * Struct used for handling a launched UE client/server process
 */
struct FUnitTestProcess
{
	friend class UClientUnitTest;

	/** Process handle for the launched process */
	FProcHandle ProcessHandle;

	/** The process ID */
	uint32 ProcessID;

	/** The suspend state of the process (implemented as a part of unit test code, does not relate to OS API) */
	ESuspendState SuspendState;

	/** Human-readable tag given to this process. Processes should be given an easily distinguishable tag, to identify them in code. */
	FString ProcessTag;


	/** Handle to StdOut for the launched process */
	void* ReadPipe;

	/** Handle to StdIn for the launched process (unused) */
	void* WritePipe;


	/** The base log type for this process (client? server? process?) */
	ELogType BaseLogType;

	/** The prefix to use for StdOut log output */
	FString LogPrefix;

	/** The OutputDeviceColor string, to use for setting the log output color */
	const TCHAR* MainLogColor;

	/** The log output color to use in the slate log window */
	FSlateColor SlateLogColor;

	/** Whether or not to strip Error logs for this process (e.g. for unit test servers, when we expect to crash) */
	bool bStripErrorLogs;


	/** If this process is outputting an error log, this is the current stage of error parsing (or ELS_NoError if not parsing) */
	EErrorLogStage ErrorLogStage;

	/** Gathered error log text */
	TArray<FErrorLog> ErrorText;


	/**
	 * Base constructor
	 */
	FUnitTestProcess()
		: ProcessHandle()
		, ProcessID(0)
		, SuspendState(ESuspendState::Active)
		, ProcessTag(TEXT(""))
		, ReadPipe(NULL)
		, WritePipe(NULL)
		, BaseLogType(ELogType::None)
		, LogPrefix(TEXT(""))
		, MainLogColor(COLOR_NONE)
		, SlateLogColor(FSlateColor::UseForeground())
		, bStripErrorLogs(false)
		, ErrorLogStage(EErrorLogStage::ELS_NoError)
		, ErrorText()
	{
	}
};


// Delegate definitions

/**
 * Delegate notifying that a process suspend state has changed
 *
 * @param NewSuspendState	The new process suspend state
 */
DECLARE_DELEGATE_OneParam(FOnSuspendStateChange, ESuspendState /*NewSuspendState*/);

/**
 * Delegate for implementing process log watches
 *
 * @param InProcess		The process the log originates from
 * @param InLogLines	The list of log lines to be processed
 * @return				Whether or not the log watch is finished - removes the watch delegate
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FProcessLogWatch, TWeakPtr<FUnitTestProcess> /*InProcess*/,
									const TArray<FString>& /*InLogLines*/);


/**
 * Base class for all unit tests which launch child processes, whether they be UE child processes, or other arbitrary programs.
 *
 * Handles management of child processes, memory usage tracking, log/stdout output gathering/printing, and crash detection.
 */
UCLASS(Abstract)
class NETCODEUNITTEST_API UProcessUnitTest : public UUnitTest
{
	GENERATED_UCLASS_BODY()

	/** Runtime variables */
protected:
	/** Stores a reference to all running child processes tied to this unit test, for housekeeping */
	TArray<TSharedPtr<FUnitTestProcess>> ActiveProcesses;

	/** Last time there was a check for processes blocking progress */
	double LastBlockingProcessCheck;

public:
	/** Delegate for notifying the UI, of a change in the unit test suspend state */
	FOnSuspendStateChange OnSuspendStateChange;

	/** Delegates for implementing process log watches */
	TArray<FProcessLogWatch> ProcessLogWatches;


	/**
	 * Interface for process unit tests
	 */
public:
	/**
	 * For implementation in subclasses, for helping to verify success/fail upon completion of unit tests
	 * NOTE: Not called again once VerificationState is set
	 * WARNING: Be careful when iterating InLogLines in multiple different for loops, if the sequence of detected logs is important
	 *
	 * @param InProcess		The process the log lines are from
	 * @param InLogLines	The current log lines being received
	 */
	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines);

	/**
	 * Notifies that there was a request to suspend/resume the unit test
	 */
	virtual void NotifySuspendRequest()
	{
	}

	/**
	 * Notifies when the suspend state of a process changes
	 *
	 * @param InProcess			The process whose suspend state has changed
	 * @param InSuspendState	The new suspend state of the process
	 */
	virtual void NotifyProcessSuspendState(TWeakPtr<FUnitTestProcess> InProcess, ESuspendState InSuspendState);

	/**
	 * Notifies when a running process is detected as having finished/closed
	 * NOTE: This will not get called, when ShutdownUnitTestProcess is used, only when a program ends by itself
	 *
	 * @param InProcess		The process which has finished
	 */
	virtual void NotifyProcessFinished(TWeakPtr<FUnitTestProcess> InProcess)
	{
	}


protected:
	/**
	 * Starts a child process, tied to the unit test
	 *
	 * @param Path				The path to the process executable
	 * @param InCommandline		The commandline that the child process should use
	 * @param bMinimized		Starts the process with the window minimized
	 * @return					Returns a pointer to the new processes handling struct
	 */
	virtual TWeakPtr<FUnitTestProcess> StartUnitTestProcess(FString Path, FString InCommandline, bool bMinimized=true);

	/**
	 * Starts a child UE process, tied to the unit test
	 *
	 * @param InCommandline		The commandline that the child process should use
	 * @param bMinimized		Starts the process with the window minimized
	 * @param Type				If we are not in Dev Editor mode, the Game/Server processes are separate - specify which here
	 * @return					Returns a pointer to the new processes handling struct
	 */
	virtual TWeakPtr<FUnitTestProcess> StartUEUnitTestProcess(FString InCommandline, bool bMinimized=true,
																EBuildTargetType Type=EBuildTargetType::Game);

	/**
	 * Shuts-down/cleans-up a child process tied to the unit test
	 *
	 * @param InHandle	The handling struct for the process
	 */
	virtual void ShutdownUnitTestProcess(TSharedPtr<FUnitTestProcess> InHandle);

	/**
	 * If any errors logs were detected upon ShutdownUnitTestProcess, this is called to print them out
	 *
	 * @param InHandle	The handling struct for the process
	 */
	virtual void PrintUnitTestProcessErrors(TSharedPtr<FUnitTestProcess> InHandle);

	/**
	 * Processes the standard output (i.e. log output) for processes
	 */
	void PollProcessOutput();

	/**
	 * Updates (and if necessary, saves) the memory stats for processes
	 */
	void UpdateProcessStats();

	/**
	 * Whether or not a child process indicating a long/blocking task is running
	 *
	 * @param bLogIfFound	Outputs a log to the unit test window, if a blocking process is found
	 * @return				Whether or not a blocking process was found
	 */
	bool IsBlockingProcessPresent(bool bLogIfFound=false);

	/**
	 * Checks incoming process logs, for any indication of a crash/error
	 *
	 * @param InProcess		The process the log output originates from
	 * @param InLines		The incoming log lines
	 */
	virtual void CheckOutputForError(TSharedPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLines);


protected:
	virtual void CleanupUnitTest(EUnitTestResetStage ResetStage) override;


	virtual void UnitTick(float DeltaTime) override;

	virtual void PostUnitTick(float DeltaTime) override;

	virtual bool IsTickable() const override;


	virtual void FinishDestroy() override;

	virtual void ShutdownAfterError() override;
};
