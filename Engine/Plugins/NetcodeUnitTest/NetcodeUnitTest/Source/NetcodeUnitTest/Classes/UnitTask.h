// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "UnitLogging.h"

#include "UnitTask.generated.h"


// Forward declarations
struct FMinClientParms;
class UUnitTest;



// @todo #JohnB: Determine if some UnitTask's will need to remain around persistently - this is quite possible, with the MCP auth stuff,
//					and needing a place to store/access the auth info

// @todo #JohnB: Consider support for execution of multiple UnitTask's, at some point, if it will be useful

// @todo #JohnB: Consider support for adding new log tabs, for UnitTask's - perhaps only if they are configured to do this,
//					or perhaps as an FUnitLogInterface ability


/**
 * Enums
 */

/**
 * Flags for specifying the required conditions for executing a UnitTask, and what stages of the unit test to block until completion
 */
enum class EUnitTaskFlags : uint8
{
	None				= 0x00,		// No flags

	/** Requirement flags */
	RequireServer		= 0x01,		// For ClientUnitTest, wait until the server has successfully started
	RequireMinClient	= 0x02,		// For ClientUnitTest, wait until the MinimalClient has successfully connected to the server

	RequirementsMask	= RequireServer | RequireMinClient,


	/** Flags for blocking unit test execution */
	BlockUnitTest		= 0x04,		// Blocks execution of the unit test at the earliest stage, preventing any setup
	BlockMinClient		= 0x08,		// For ClientUnitTest, blocks the creation/connection of the MinimalClient

	BlockMask			= BlockUnitTest | BlockMinClient,

	/** Flags for altering stages of unit test execution */
	AlterMinClient		= 0x10,		// For ClientUnitTest, alter parameters for MinimalClient setup, through NotifyAlterMinClient


	/** Masks for the sequence UnitTask's should run, with above flags (done like this, in case future flags share same priority) */
	PriorityMask_One	= None,
	PriorityMask_Two	= BlockUnitTest,
	PriorityMask_Three	= RequireServer,
	PriorityMask_Four	= BlockMinClient,
	PriorityMask_Five	= AlterMinClient,
	PriorityMask_Six	= RequireMinClient,
	PriorityMask_MAX	= PriorityMask_Six
};

ENUM_CLASS_FLAGS(EUnitTaskFlags);

/**
 * Returns the numeric priority for the specified UnitTask flags
 *
 * @return	The priority for the specified UnitTask flags
 */
int32 GetUnitTaskPriority(EUnitTaskFlags InFlags);


/**
 * UnitTask's are used to implement supporting code for UnitTest's, for handling complex behind-the-scenes setup prior to test execution
 * (e.g. primarily for implementing game-specific server/client environment setup), which is shared between many unit tests,
 * and which is better to abstract-away from visibility in unit tests themselves, for code clarity.
 *
 * For example, some games require authentication negotiation before the game client can connect to a server,
 * and this is the type of task that UnitTask's are designed for.
 *
 * Unit tasks that are added to a unit test, must complete execution before the unit test itself can execute.
 */
UCLASS(abstract)
class NETCODEUNITTEST_API UUnitTask : public UObject, public FUnitLogRedirect
{
	friend UUnitTest;

	GENERATED_UCLASS_BODY()

protected:
	/** Flags for configuring the UnitTask */
	EUnitTaskFlags UnitTaskFlags;

	/** The UnitTest that owns this task */
	UPROPERTY()
	TObjectPtr<UUnitTest> Owner;

	/** Whether or not the UnitTask has started */
	bool bStarted;

public:
	/**
	 * Attach the task to a unit test
	 *
	 * @param InOwner	The unit test that will own this task
	 */
	virtual void Attach(UUnitTest* InOwner);

	/**
	 * After attaching to a unit test, validate the unit task settings are compatible
	 */
	virtual void ValidateUnitTaskSettings();


	/**
	 * Start executing the task
	 */
	virtual void StartTask()
	{
	}

	/**
	 * Whether or not the task has started
	 *
	 * @return	Whether or not the task has started
	 */
	FORCEINLINE bool HasStarted()
	{
		return bStarted;
	}

	/**
	 * Whether or not the task has completed
	 *
	 * @return	Returns whether the task has completed or not
	 */
	virtual bool IsTaskComplete() PURE_VIRTUAL(UUnitTask::IsTaskComplete, return false;);

	/**
	 * Cleanup the task state, after the task completes
	 */
	virtual void Cleanup()
	{
	}


	/**
	 * Gives UnitTask's an opportunity to alter the MinimalClient setup parameters
	 *
	 * @param Parms		Reference to the MinimalClient parameters prior to setup/connection
	 */
	virtual void NotifyAlterMinClient(FMinClientParms& Parms)
	{
	}

	/**
	 * Triggers a failure for the UnitTask and owning UnitTest
	 *
	 * @param Reason	The reason for the failure (output to log)
	 */
	void TriggerFailure(FString Reason);

	/**
	 * See UUnitTest::ResetTimeout
	 */
	virtual void ResetTimeout(FString ResetReason, bool bResetConnTimeout=false, uint32 MinDuration=0);


	// Accessors

	/**
	 * Retrieves UnitTaskFlags
	 */
	FORCEINLINE EUnitTaskFlags GetUnitTaskFlags() const
	{
		return UnitTaskFlags;
	}


	/** Events passed from the unit test */
public:
	/**
	 * Main tick function for the unit test
	 *
	 * @param DeltaTime		The time passed since the previous tick
	 */
	virtual void UnitTick(float DeltaTime)
	{
	}

	/**
	 * Tick function that runs at a tickrate of ~60 fps, for interacting with netcode
	 * (high UnitTick tickrate, can lead to net buffer overflows)
	 */
	virtual void NetTick()
	{
	}

	// Must override in subclasses, that need ticking
	virtual bool IsTickable() const
	{
		return false;
	}
};
