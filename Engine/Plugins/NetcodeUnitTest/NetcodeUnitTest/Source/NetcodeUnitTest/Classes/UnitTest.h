// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Containers/ArrayView.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceFile.h"
#include "Styling/SlateColor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "NetcodeUnitTest.h"
#include "UnitTestBase.h"
#include "UnitTask.h"

#include "UnitTest.generated.h"


// @todo #JohnBFeature: For bugtracking/changelist info, consider adding auto-launching of P4/TTP/Browser-JIRA links,
//				upon double-clicking these entries in the status windows


// Forward declarations
class FUnitTestEnvironment;
class SLogWindow;


/**
 * Enums
 */

/**
 * The verification status of the current unit test - normally its execution completes immediately after positive/negative verification
 */
UENUM()
enum class EUnitTestVerification : uint8
{
	/** Unit test is not yet verified */
	Unverified,
	/** Unit test is verified as not fixed */
	VerifiedNotFixed,
	/** Unit test is verified as fixed */
	VerifiedFixed,
	/** Unit test is no longer functioning, needs manual check/update (issue may be fixed, or unit test broken) */
	VerifiedNeedsUpdate,
	/** Unit test is verified as having executed unreliably */
	VerifiedUnreliable,
};

/**
 * The different stages that unit tests can be reset to - a global/non-locally-customizable list, for now
 * NOTE: Stages MUST be sequential! (e.g. ResetConnection implies ResetExecute, FullReset implies both ResetConnection and ResetExecute)
 * NOTE: Apart from checking for 'None', all comparisons should be either <= or >=, to support potential enum additions
 */
UENUM()
enum class EUnitTestResetStage : uint8
{
	/** No reset stage */
	None,
	/** Resets the entire unit test, allowing restart from the beginning */
	FullReset,
	/** For ClientUnitTest's, resets the net connection and minimal client - but not the server - allowing a restart from connecting */
	ResetConnection,
	/** Resets unit tests to the point prior to 'ExecuteUnitTest' - usually implemented individually per unit test */
	ResetExecute
};

FString GetUnitTestResetStageName(EUnitTestResetStage Stage);


/**
 * Structs
 */

/**
 * Used for storing unit-test-specific logs, which are displayed in the status window
 * (upon completion of unit testing, a final summary is printed using this data, but in a more-ordered/easier-to-read fashion)
 */
struct FUnitStatusLog
{
	/** The log type for this status log */
	ELogType	LogType;

	/** The log line */
	FString		LogLine;


	FUnitStatusLog()
		: LogType(ELogType::None)
		, LogLine(TEXT(""))
	{
	}

	FUnitStatusLog(ELogType InLogType, const FString& InLogLine)
		: LogType(InLogType)
		, LogLine(InLogLine)
	{
	}
};


/**
 * Base class for all unit tests
 */
UCLASS(Abstract, config=UnitTestStats)
class NETCODEUNITTEST_API UUnitTest : public UUnitTestBase
{
	GENERATED_UCLASS_BODY()

	friend class UUnitTestManager;
	friend class FUnitTestEnvironment;
	friend class UMinimalClient;
	friend class UUnitTask;
	friend struct NUTNet;


	/** Variables which should be specified by every subclass */
protected:
	/** The name/command for this unit test (N.B. Must be set in class constructor) */
	FString UnitTestName;

	/** The type of unit test this is (e.g. bug/exploit) (N.B. Must be set in class constructor) */
	FString UnitTestType;

	/** The date this unit test was added to the project (for ordering in help command) */
	FDateTime UnitTestDate;


	/** The bug tracking identifiers related to this unit test (e.g. TTP numbers) */
	TArray<FString> UnitTestBugTrackIDs;

	/** Source control changelists relevant to this unit test */
	TArray<FString> UnitTestCLs;


	/** Whether or not this unit test is a 'work in progress', and should not be included in automated tests */
	bool bWorkInProgress;

	/** Whether or not this unit test is unreliable, i.e. prone to giving incorrect/unexpected results, requiring multiple runs */
	bool bUnreliable;

	/** Whether or not this unit test is obsolete - i.e. based on code no longer present in the game/engine */
	bool bObsolete;


	/**
	 * The unit test result we expect for each games codebase, i.e. whether we expect that the problem is fixed yet or not
	 * NOTE: Games which don't have an expected result specified here, are considered 'unsupported' and the unit test isn't run for them
	 */
	TMap<FString, EUnitTestVerification> ExpectedResult;


	/** The amount of time (in seconds), before the unit test should timeout and be marked as broken */
	uint32 UnitTestTimeout;


	/** Config variables */
public:
	/** Stores stats on the highest-ever reported memory usage, for this unit test - for estimating memory usage */
	UPROPERTY(config)
	uint64 PeakMemoryUsage;

	/** The amount of time it takes to reach 'PeakMemoryUsage' (or within 90% of its value) */
	UPROPERTY(config)
	float TimeToPeakMem;

	/** The amount of time it took to execute the unit test the last time it was run */
	UPROPERTY(config)
	float LastExecutionTime;



	/** Runtime variables */
protected:
	/** The unit test environment (not set until the current games unit test module is loaded - not set at all, if no such module) */
	static FUnitTestEnvironment* UnitEnv;

	/** The null unit test environment - for unit tests which support all games, due to requiring no game-specific features */
	static FUnitTestEnvironment* NullUnitEnv;


	/** The time of the last NetTick event */
	double LastNetTick;

	/** The current realtime memory usage of the unit test */
	uint64 CurrentMemoryUsage;


	/** The time at which execution of the unit test started */
	double StartTime;


	/** The time at which the unit test timeout will expire */
	double TimeoutExpire;

	/** The last time that the unit test timeout was reset */
	double LastTimeoutReset;

	/** Every timeout reset specifies a string to identify/describe the event that triggered it, for tracking */
	FString LastTimeoutResetEvent;

	/** Whether or not developer-mode has been enabled for this unit test (prevents it from ending execution) */
	bool bDeveloperMode;


	/** Whether it's the first time this unit test has run, i.e. whether prior memory stats exist (NOTE: Not set until first tick) */
	bool bFirstTimeStats;


	/** UnitTask's which must be run before different stages of the unit test can execute */
	UPROPERTY()
	TArray<TObjectPtr<UUnitTask>> UnitTasks;

	/** Marks the state of met unit task requirement flags and active unit task blocking flags */
	EUnitTaskFlags UnitTaskState;


	// @todo #JohnBRefactor: Merge the two below variables
	/** Whether or not the unit test has completed */
	bool bCompleted;

	/** Whether or not the success or failure of the current unit test has been verified */
	UPROPERTY()
	EUnitTestVerification VerificationState;

private:
	/** Whether or not the verification state was already logged (prevent spamming in developer mode) */
	bool bVerificationLogged;

protected:
	/** Whether or not the unit test has aborted execution */
	bool bAborted;


	/** The log window associated with this unit test */
	TSharedPtr<SLogWindow> LogWindow;

	/** Collects unit test status logs, that have been printed to the summary window */
	TArray<TSharedPtr<FUnitStatusLog>> StatusLogSummary;


	/** The log file for outputting all log information for the current unit test */
	TUniquePtr<FOutputDeviceFile> UnitLog;

	/** The log directory for this unit test */
	FString UnitLogDir;


public:
	/**
	 * Returns the name/command, for the current unit test
	 */
	FORCEINLINE FString GetUnitTestName() const
	{
		return UnitTestName;
	}

	/**
	 * Returns the type of unit test (e.g. bug/exploit)
	 */
	FORCEINLINE FString GetUnitTestType() const
	{
		return UnitTestType;
	}

	/**
	 * Returns the date this unit test was first added to the code
	 */
	FORCEINLINE FDateTime GetUnitTestDate() const
	{
		return UnitTestDate;
	}

	/**
	 * Returns the value of UnitTestTimeout
	 */
	FORCEINLINE uint32 GetUnitTestTimeout() const
	{
		return UnitTestTimeout;
	}

	/**
	 * Returns the expected result for the current game
	 */
	FORCEINLINE EUnitTestVerification GetExpectedResult()
	{
		EUnitTestVerification Result = EUnitTestVerification::Unverified;

		FString CurGame = FApp::GetProjectName();

		if (ExpectedResult.Contains(CurGame))
		{
			Result = ExpectedResult[CurGame];
		}
		else if (ExpectedResult.Contains(TEXT("NullUnitEnv")))
		{
			Result = ExpectedResult[TEXT("NullUnitEnv")];
		}

		return Result;
	}

	/**
	 * Returns the list of supported games, for this unit test
	 */
	FORCEINLINE TArray<FString> GetSupportedGames()
	{
		TArray<FString> SupportedGames;

		ExpectedResult.GenerateKeyArray(SupportedGames);

		return SupportedGames;
	}


	/**
	 * Returns whether or not this is the first time the unit test has been run/collecting-stats
	 */
	FORCEINLINE bool IsFirstTimeStats() const
	{
		return bFirstTimeStats || PeakMemoryUsage == 0;
	}


protected:
	/**
	 * Finishes initializing unit test settings, that rely upon the current unit test environment being loaded
	 */
	virtual void InitializeEnvironmentSettings()
	{
	}

	/**
	 * Validate that the unit test settings/flags specified for this unit test, are compatible with one another,
	 * and that the engine settings/environment, support running the unit test.
	 *
	 * @param bCDOCheck		If true, the class default object is being validated before running - don't check class runtime variables
	 * @return				Returns true, if all settings etc. check out
	 */
	virtual bool ValidateUnitTestSettings(bool bCDOCheck=false);

	/**
	 * Returns the type of log entries that this unit expects to output, for setting up log window filters
	 * (only needs to return values which affect what tabs are shown)
	 *
	 * @return		The log type mask, representing the type of log entries this unit test expects to output
	 */
	virtual ELogType GetExpectedLogTypes()
	{
		return ELogType::Local;
	}

	/**
	 * Resets the unit test timeout code - should be used liberally, within every unit test, when progress is made during execution
	 *
	 * @param ResetReason			Human-readable reason for resetting the timeout
	 * @param bResetConnTimeout		Whether or not to also reset timeouts on unit test connections
	 * @param MinDuration			The minimum timeout duration (used in subclasses, to override the timeout duration)
	 */
	virtual void ResetTimeout(FString ResetReason, bool bResetConnTimeout=false, uint32 MinDuration=0)
	{
		uint32 CurrentTimeout = FMath::Max(MinDuration, UnitTestTimeout);
		double NewTimeoutExpire = FPlatformTime::Seconds() + (double)CurrentTimeout;

		// Don't reset to a shorter timeout, than is already in place
		TimeoutExpire = FMath::Max(NewTimeoutExpire, TimeoutExpire);

		LastTimeoutReset = FPlatformTime::Seconds();
		LastTimeoutResetEvent = ResetReason;
	}

	virtual bool UTStartUnitTest() override final;

	/**
	 * Sets up the log directory and log output device instances.
	 */
	void InitializeLogs();


	/**
	 * Determines whether or not a UnitTask is blocking the specified event
	 *
	 * @return	Whether or not the specified event is being blocked by a UnitTask
	 */
	FORCEINLINE bool IsTaskBlocking(EUnitTaskFlags InFlag)
	{
		bool bReturnVal = false;

		check((InFlag & EUnitTaskFlags::BlockMask) == InFlag);

		for (UUnitTask* CurTask : UnitTasks)
		{
			if (!!(CurTask->GetUnitTaskFlags() & InFlag))
			{
				bReturnVal = true;
				break;
			}
		}

		return bReturnVal;
	}

	/**
	 * When events that were pending but blocked by a UnitTask, are unblocked, this function triggers them.
	 *
	 * @param ReadyEvents	Blocked events that are ready to be unblocked/triggered
	 */
	virtual void UnblockEvents(EUnitTaskFlags ReadyEvents);

	/**
	 * Triggered when a UnitTask fails unrecoverably during execution
	 *
	 * @param InTask	The UnitTask that failed
	 * @param Reason	The reason for the failure
	 */
	virtual void NotifyUnitTaskFailure(UUnitTask* InTask, FString Reason);

public:
	/**
	 * Whether or not the unit test has started
	 *
	 * @return	Whether or not the unit test has started
	 */
	FORCEINLINE bool HasStarted()
	{
		return StartTime != 0.0;
	}

	/**
	 * Adds a UnitTask to the unit test, during its configuration stage - to be completed before executing the unit test
	 *
	 * @param InTask	The task to be added (must already have its flags etc. set)
	 */
	void AddTask(UUnitTask* InTask);


	/**
	 * Executes the main unit test
	 *
	 * @return	Whether or not the unit test kicked off execution successfully
	 */
	virtual bool ExecuteUnitTest() PURE_VIRTUAL(UUnitTest::ExecuteUnitTest, return false;)

	/**
	 * Aborts execution of the unit test, part-way through
	 */
	void AbortUnitTest();

	/**
	 * Called upon completion of the unit test (may not happen during same tick), for tearing down created worlds/connections/etc.
	 * NOTE: Should be called last, in overridden functions, as this triggers deletion of the unit test object
	 */
	void EndUnitTest();

	/**
	 * Cleans up all items needing destruction, and removes the unit test from tracking, before deleting the unit test itself
	 *
	 * @param ResetStage	If called from ResetUnitTest, restricts what is cleaned up based on the current reset stage
	 */
	virtual void CleanupUnitTest(EUnitTestResetStage ResetStage=EUnitTestResetStage::None);


	/**
	 * Resets the unit test to its initial state, allowing it to restart from scratch
	 * NOTE: Must be implemented for every unit test that intends to support it, through CleanupUnitTest
	 *
	 * @param ResetStage	If specified, only partially resets the unit test state, to allow replaying only a portion of the unit test
	 */
	void ResetUnitTest(EUnitTestResetStage ResetStage=EUnitTestResetStage::FullReset);

	/**
	 * Whether or not this unit test supports resetting
	 */
	virtual bool CanResetUnitTest()
	{
		return false;
	}


	virtual void NotifyLocalLog(ELogType InLogType, const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
		override;

	virtual void NotifyStatusLog(ELogType InLogType, const TCHAR* Data) override;

	virtual bool IsConnectionLogSource(UNetConnection* InConnection) override;

	virtual bool IsTimerLogSource(UObject* InTimerDelegateObject) override;


	/**
	 * Notifies that there was a request to enable/disable developer mode
	 *
	 * @param bInDeveloperMode	Whether or not developer mode is being enabled/disabled
	 */
	void NotifyDeveloperModeRequest(bool bInDeveloperMode);

	/**
	 * Notifies that there was a request to execute a console command for the unit test, which can occur in a specific context,
	 * e.g. for a unit test server, for a local minimal-client (within the unit test), or for a separate unit test client process
	 *
	 * @param CommandContext	The context (local/server/client?) for the console command
	 * @param Command			The command to be executed
	 * @return					Whether or not the command was handled
	 */
	virtual bool NotifyConsoleCommandRequest(FString CommandContext, FString Command);

	/**
	 * Outputs the list of console command contexts, that this unit test supports (which can include custom contexts in subclasses)
	 *
	 * @param OutList				Outputs the list of supported console command contexts
	 * @param OutDefaultContext		Outputs the context which should be auto-selected/defaulted-to
	 */
	virtual void GetCommandContextList(TArray<TSharedPtr<FString>>& OutList, FString& OutDefaultContext);


	virtual void UnitTick(float DeltaTime) override;

	virtual void NetTick() override;

	virtual void PostUnitTick(float DeltaTime) override;

	virtual bool IsTickable() const override;

	virtual void TickIsComplete(float DeltaTime) override;

	/**
	 * Triggered upon unit test completion, for outputting that the unit test has completed - plus other unit test state information
	 */
	virtual void LogComplete();
};
