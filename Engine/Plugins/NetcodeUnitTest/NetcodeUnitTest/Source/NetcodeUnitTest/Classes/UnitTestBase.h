// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "UnitLogging.h"

#include "UnitTestBase.generated.h"

/**
 * Base class for the unit test framework event implementation
 * (all engine/external-triggered events are wrapped, in order to hook logs triggered during their execution)
 *
 * NOTE: All wrapped functions/events, begin with 'UT'
 */
UCLASS(abstract)
class NETCODEUNITTEST_API UUnitTestBase : public UObject, public FUnitLogInterface
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Adds the unit test to tracking, and starts it
	 *
	 * @return	Whether or not the unit test was started successfully
	 */
	bool StartUnitTest();

	virtual bool UTStartUnitTest() PURE_VIRTUAL(UUnitTestBase::UTStartUnitTest, return false;);


	/**
	 * Main tick function for the unit test
	 *
	 * @param DeltaTime		The time passed since the previous tick
	 */
	virtual void UnitTick(float DeltaTime)
	{
	}

	/**
	 * Tick function that runs at a tickrate of ~60 fps, for interacting with netcode
	 * (high UnitTick tickrate, can lead to net buffer overflows)
	 */
	virtual void NetTick()
	{
	}

	/**
	 * For cleanup actions that should occur after the primary tick function is called
	 *
	 * @param DeltaTime		The time passed since the previous tick
	 */
	virtual void PostUnitTick(float DeltaTime)
	{
	}

	/**
	 * Tick function for checking if the unit test is completed (happens after all above tick events)
	 *
	 * @param DeltaTime		The time passed since the previous tick
	 */
	virtual void TickIsComplete(float DeltaTime)
	{
	}

	// Must override in subclasses, that need ticking
	virtual bool IsTickable() const
	{
		return false;
	}
};
