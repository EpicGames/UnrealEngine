// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"

#include "UnitTestCommandlet.generated.h"

/**
 * A commandlet for running unit tests, without having to launch the game client.
 * Uses a heavily stripped down game client, with allowances for Slate, and some hacks for enabling proper netcode usage.
 *
 * Usage:
 *	"UnrealEditor.exe shootergame -run=NetcodeUnitTest.UnitTestCommandlet"
 *
 * Parameters:
 *	-UnitTest=UnitTestName
 *		- Launches only the specified unit test
 *
 *	-UnitTestNoAutoClose
 *		- Sets the default option for the 'AutoClose' button to False
 *
 *	-UnitTestServerParms="CommandlineParameters"
 *		- Adds additional commandline parameters to unit test server instances (useful for e.g. unsuppressing specific logs)
 *
 *	-UnitTestClientParms="CommandlineParameters"
 *		- Adds additional commandline parameters to unit test client instances
 *
 *	-UnitTestClientDebug
 *		- Launches unit test clients with a valid render interface (so you can interact with the client window), and a larger window
 *
 *	-UnitTestCap=x
 *		- Caps the maximum number of unit tests that can run at the same time
 */
UCLASS()
class UUnitTestCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	virtual int32 Main(const FString& Params) override;

	virtual void CreateCustomEngine(const FString& Params) override;
};
