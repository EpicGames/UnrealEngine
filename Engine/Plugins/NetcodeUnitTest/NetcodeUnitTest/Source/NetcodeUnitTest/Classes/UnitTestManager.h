// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/OutputDeviceFile.h"
#include "Styling/SlateColor.h"
#include "UnrealClient.h"
#include "Tickable.h"
#include "UnitTestManager.generated.h"

// Forward declarations
class FLogWindowManager;
class SLogWindow;
class SWindow;
class UUnitTest;

/**
 * Manages centralized execution and tracking of unit tests, as well as handling console commands,
 * and some misc tasks like local log hooking
 */
UCLASS(config=UnitTest)
class UUnitTestManager : public UObject, public FTickableGameObject, public FOutputDevice
{
	GENERATED_UCLASS_BODY()

public:
	// NOTE: When adding new config values, add their default values for .ini creation, into the Initialize function

	/** Whether or not to cap the maximum number of unit tests that can be active at any given time */
	UPROPERTY(config)
	bool bCapUnitTestCount;

	/** Specifies the maximum number of unit tests that can be run at any given time */
	UPROPERTY(config)
	uint8 MaxUnitTestCount;


	/** Whether or not to cap active unit tests, based on memory usage */
	UPROPERTY(config)
	bool bCapUnitTestMemory;

	// @todo #JohnBLowPri: Add a bool, for specifying that unit tests can be terminated, when memory limits are breached

	/** When total physical memory usage, as a percentage, reaches this limit, no new unit tests can be started */
	UPROPERTY(config)
	uint8 MaxMemoryPercent;

	/** As above, but when reaching this limit, recently started unit test(s) will be terminated/re-queued, to get back within limits */
	UPROPERTY(config)
	uint8 AutoCloseMemoryPercent;

	/** Limits the number of auto-aborts a particular unit test will allow, before it is no longer accepted for re-queueing */
	UPROPERTY(config)
	uint8 MaxAutoCloseCount;


	/** The number of recorded sessions where UE has run unit tests (max one per each run of the UE process) */
	UPROPERTY(config)
	uint32 UnitTestSessionCount;


	/** Holds a list of unit tests pending execution */
	UPROPERTY()
	TArray<TObjectPtr<UClass>> PendingUnitTests;

	/** Holds a list of currently active unit tests */
	UPROPERTY()
	TArray<TObjectPtr<UUnitTest>> ActiveUnitTests;

	/** Unit tests which are finished, and are kept around until printing the final summary */
	UPROPERTY()
	TArray<TObjectPtr<UUnitTest>> FinishedUnitTests;


	/** If a unit test was aborted on its first run, strictly cap all first-run unit tests to one at a time */
	bool bAbortedFirstRunUnitTest;

	/** Whether or not to allow re-queuing of unit tests */
	bool bAllowRequeuingUnitTests;


private:
	/** The log window manager - used for creating and managing the positioning of unit test log windows */
	FLogWindowManager* LogWindowManager;

	/** Whether the current log line being written, is a status log or not (used to toggle Serialize function between modes) */
	bool bStatusLog;

	/** The colour to use for the current status log */
	FSlateColor StatusColor;

	/** Maps open dialog boxes, to unit tests */
	TMap<TSharedRef<SWindow>, UUnitTest*> DialogWindows;

public:
	/** The log window which displays the overall status of unit testing */
	TSharedPtr<SLogWindow> StatusWindow;

	/** The 'abort all' dialog */
	TSharedPtr<SWindow> AbortAllDialog;


	/** The log file for outputting overall unit test status */
	TUniquePtr<FOutputDeviceFile> StatusLog;


private:
	/** The base log directory used by unit tests, for this session */
	FString BaseUnitLogDir;

	/** The time at which the memory limit was last hit */
	double LastMemoryLimitHit;

	/** When a unit test is force-closed, wait a number of ticks for global memory values to update, before closing any more */
	int32 MemoryTickCountdown;

	/** When waiting for restart of unit test auto-closing, note the system memory usage, and end the countdown early if it increases */
	SIZE_T MemoryUsageUponCountdown;

public:
	/**
	 * Static getter for the unit test manager
	 *
	 * @return	Returns the unit test manager
	 */
	static UUnitTestManager* Get();

	/**
	 * Initialize the unit test manager
	 */
	void Initialize();

	/**
	 * Initialize unit test log output
	 */
	void InitializeLogs();

	/**
	 * Returns the base log directory used by unit tests
	 */
	FORCEINLINE const FString GetBaseUnitLogDir()
	{
		return BaseUnitLogDir;
	}

	/**
	 * Destructor for handling removal of log registration
	 */
	virtual ~UUnitTestManager() override;


	/**
	 * Queues a unit test for execution
	 *
	 * @param UnitTestClass	The unit test class to be queued
	 * @param bRequeued		Whether the unit test being queued, was aborted and is now being requeued
	 * @return				Whether or not queueing was successful
	 */
	bool QueueUnitTest(UClass* UnitTestClass, bool bRequeued=false);

	/**
	 * Checks to see if we're ready to execute any unit tests in the queue, and if so, begins execution
	 */
	void PollUnitTestQueue();

	/**
	 * Tests whether currently active unit tests, and optionally/additionally, a unit test about to be executed,
	 * fall within limits/restrictions on unit test counts and memory usage
	 *
	 * @param PendingUnitTest	Optionally specify a unit test pending execution, to estimate and factor in its resource usage
	 * @return					Returns true if within limits, and false if not
	 */
	bool WithinUnitTestLimits(UClass* PendingUnitTest=NULL);

	/**
	 * Returns whether or not there are unit tests running (or about to be run)
	 */
	bool IsRunningUnitTests()
	{
		return ActiveUnitTests.Num() > 0 || PendingUnitTests.Num() > 0;
	}


	/**
	 * Notification that is triggered when a unit test completes
	 *
	 * @param InUnitTest	The unit test that just completed
	 * @param bAborted		Whether or not the unit test was aborted
	 */
	void NotifyUnitTestComplete(UUnitTest* InUnitTest, bool bAborted);

	/**
	 * Notification that is triggered when a unit test is cleaning up
	 *
	 * @param InUnitTest	The unit test that is cleaning up
	 */
	void NotifyUnitTestCleanup(UUnitTest* InUnitTest);

	/**
	 * Notifies when a log window has closed
	 *
	 * @param ClosedWindow	The log window that has closed
	 */
	void NotifyLogWindowClosed(const TSharedRef<SWindow>& ClosedWindow);


	/**
	 * When a log window is closed, a dialog pops up asking if the unit test should be aborted - this returns the result
	 * NOTE: This dialog is non-modal, i.e. doesn't block the game thread - so the unit test is not guaranteed to still exist
	 *
	 * @param DialogWindow	The dialog window the result is coming from
	 * @param Result		The result (yes/no) for the dialog
	 * @param bNoResult		Whether or not the result is invalid, e.g. the dialog was closed instead of a button clicked
	 */
	void NotifyCloseDialogResult(const TSharedRef<SWindow>& DialogWindow, EAppReturnType::Type Result, bool bNoResult);

	/**
	 * When the main status window is closed, a dialog pops up asking if you want to abort all running unit tests;
	 * this returns the result
	 *
	 * @param DialogWindow	The dialog window the result is coming from
	 * @param Result		The result (yes/no) for the dialog
	 * @param bNoResult		Whether or not the result is invalid, e.g. the dialog was closed instead of a button clicked
	 */
	void NotifyCloseAllDialogResult(const TSharedRef<SWindow>& DialogWindow, EAppReturnType::Type Result, bool bNoResult);


	/**
	 * Dumps status information for running/pending unit tests, to the status window and log
	 *
	 * @param bForce	Forces a stats dump
	 */
	void DumpStatus(bool bForce=false);

	/**
	 * Prints the results information for a single unit test
	 *
	 * @param InUnitTest		The unit test to print results information for
	 * @param bFinalSummary		Whether or not this is the final summary printout (changes the formatting slightly)
	 * @param bUnfinished		Whether or not the unit test was aborted and could not be run
	 */
	void PrintUnitTestResult(UUnitTest* InUnitTest, bool bFinalSummary=false, bool bUnfinished=false);

	/**
	 * Prints the final unit test summary, when all active/pending unit tests have completed
	 */
	void PrintFinalSummary();


	/**
	 * Opens the log window, for a unit test
	 *
	 * @param InUnitTest	The unit test associated with the log window
	 */
	void OpenUnitTestLogWindow(UUnitTest* InUnitTest);

	/**
	 * Opens the status log window
	 */
	void OpenStatusWindow();


	/**
	 * FTickableGameObject methods
	 */

	virtual void Tick(float DeltaTime) override;

	// Must override in subclasses, that need ticking
	virtual bool IsTickable() const override
	{
		// @todo #JohnBLowPri: Find out how the CDO is getting registered for ticking - this is odd
		return IsValidChecked(this) && !GIsServer && !HasAnyFlags(RF_ClassDefaultObject);
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}

	virtual TStatId GetStatId() const override
	{
		return TStatId();
	}


	/**
	 * Handles exec commands starting with 'UnitTest'
	 *
	 * @param InWorld	The world the command was executed from
	 * @param Cmd		The command that was executed
	 * @param Ar		The archive log results should be sent to
	 * @return			True if the command was handled, false if not
	 */
	bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

	/**
	 * FOutputDevice methods
	 */

	// We're hiding UObject::Serialize() by declaring this.  That's OK, but Clang will warn about it.
#if defined(__clang__)
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Woverloaded-virtual"
#endif
	/**
	 * Write text to the console
	 *
	 * @param Data		The text to write
	 * @param Verbosity	The log level verbosity for the message
	 * @param Category	The log category for the message
	 */
	virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override;
#if defined(__clang__)
#pragma clang diagnostic pop
#endif


	/**
	 * Getters/Setters
	 */

	FORCEINLINE void SetStatusLog(bool bInStatusLog)
	{
		bStatusLog = bInStatusLog;
	}

	FORCEINLINE void SetStatusColor(FSlateColor InStatusColor=GetDefaultStatusColor())
	{
		StatusColor = InStatusColor;
	}

	static FORCEINLINE FSlateColor GetDefaultStatusColor()
	{
		return FSlateColor::UseForeground();
	}
};
