// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "ClientUnitTest.h"

#include "IPClient.generated.h"


/**
 * Basic unit test for launching a server and connecting a client, while verifying that the correct net driver was used,
 * and that the client connected successfully.
 */
UCLASS()
class UIPClient : public UClientUnitTest
{
	GENERATED_UCLASS_BODY()

protected:
	/** Log message identifying the expected net driver */
	const TCHAR* NetDriverLog;

private:
	/** Log message triggered upon successful connect */
	const TCHAR* ConnectSuccessMsg;

	/** Whether or not the correct net driver was detected */
	bool bDetectedNetDriver;

public:
	virtual void InitializeEnvironmentSettings() override;

	virtual void ExecuteClientUnitTest() override;

	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;
};
