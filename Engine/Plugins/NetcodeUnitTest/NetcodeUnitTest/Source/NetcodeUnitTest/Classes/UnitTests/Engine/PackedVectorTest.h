// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UnitTest.h"

#include "PackedVectorTest.generated.h"


/**
 * Basic unit test for WritePackedValue and ReadPackedValue
 */
UCLASS()
class UPackedVectorTest : public UUnitTest
{
	GENERATED_UCLASS_BODY()

private:
	virtual bool ExecuteUnitTest() override;

	bool ExecuteFloatTest(TMap<FString,bool>& TestResults);
	bool ExecuteDoubleTest(TMap<FString,bool>& TestResults);
	bool ExecuteWriteDoubleReadFloatTest(TMap<FString,bool>& TestResults);
	bool ExecuteWriteFloatReadDoubleTest(TMap<FString,bool>& TestResults);
	void VerifyTestResults(const TMap<FString,bool>& TestResults);
};
