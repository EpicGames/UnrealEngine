// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnitTests/Engine/IPClient.h"

#include "SteamClient.generated.h"

/**
 * Basic unit test for verifying simple client connection to a server, using the Steam net driver.
 */
UCLASS()
class USteamClient : public UIPClient
{
	GENERATED_UCLASS_BODY()

protected:
	/** The server Steam address, parsed from the server log */
	FString ServerSteamAddress;

	/** Whether or not ConnectMinimalClient is pending */
	bool bPendingConnect;

protected:
	virtual bool ValidateUnitTestSettings(bool bCDOCheck) override;

	virtual void InitializeEnvironmentSettings() override;

	virtual FString ConstructServerParameters() override;

	virtual bool ConnectMinimalClient(const TCHAR* InNetID=nullptr) override;

	virtual void NotifyAlterMinClient(FMinClientParms& Parms) override;

	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;
};
