// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnitTests/Engine/IPClient.h"

#include "WebSocketClient.generated.h"

/**
 * Basic unit test for verifying simple client connection to a server, using the WebSocket net driver.
 */
UCLASS()
class UWebSocketClient : public UIPClient
{
	GENERATED_UCLASS_BODY()

protected:
	virtual void InitializeEnvironmentSettings() override;

	virtual void NotifyAlterMinClient(FMinClientParms& Parms) override;
};
