// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UnitTest.h"
#include "UObject/ObjectPtr.h"

#include "VMReflection.generated.h"


// Forward declarations
class APawn;

/**
 * Internal unit test for verifying the functionality of the UScript/BP VM reflection helper
 */
UCLASS()
class UVMReflection : public UUnitTest
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool ExecuteUnitTest() override;
};

/**
 * Test classes for testing different types/combinations of property reflection
 */

UCLASS()
class UVMTestClassA : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY()
	TObjectPtr<UObject>	AObjectRef;

	UPROPERTY()
	uint8		ByteProp;

	UPROPERTY()
	uint16		UInt16Prop;

	UPROPERTY()
	uint32		UInt32Prop;

	UPROPERTY()
	uint64		UInt64Prop;

	UPROPERTY()
	int8		Int8Prop;

	UPROPERTY()
	int16		Int16Prop;

	UPROPERTY()
	int32		Int32Prop;

	UPROPERTY()
	int64		Int64Prop;

	UPROPERTY()
	float		FloatProp;

	UPROPERTY()
	double		DoubleProp;

	UPROPERTY()
	bool		bBoolPropA;

	UPROPERTY()
	bool		bBoolPropB;

	UPROPERTY()
	bool		bBoolPropC;

	UPROPERTY()
	bool		bBoolPropD;

	UPROPERTY()
	bool		bBoolPropE;

	UPROPERTY()
	FName		NameProp;

	UPROPERTY()
	FString		StringProp;

	UPROPERTY()
	FText		TextProp;


	UPROPERTY()
	uint8		BytePropArray[4];

	UPROPERTY()
	TObjectPtr<UObject>	ObjectPropArray[2];

	UPROPERTY()
	TArray<uint8>		DynBytePropArray;

	UPROPERTY()
	TArray<bool>		DynBoolPropArray;

	UPROPERTY()
	TArray<TObjectPtr<UObject>>	DynObjectPropArray;

	UPROPERTY()
	TArray<FName>		DynNamePropArray;

	UPROPERTY()
	TArray<double>		DynDoublePropArray;

	UPROPERTY()
	TArray<float>		DynFloatPropArray;

	UPROPERTY()
	TArray<int16>		DynInt16PropArray;

	UPROPERTY()
	TArray<int64>		DynInt64PropArray;

	UPROPERTY()
	TArray<int8>		DynInt8PropArray;

	UPROPERTY()
	TArray<int32>		DynIntPropArray;

	UPROPERTY()
	TArray<uint16>		DynUInt16PropArray;

	UPROPERTY()
	TArray<uint32>		DynUIntPropArray;

	UPROPERTY()
	TArray<uint64>		DynUInt64PropArray;

	UPROPERTY()
	TArray<FString>		DynStringPropArray;

	UPROPERTY()
	TArray<FText>		DynTextPropArray;

	UPROPERTY()
	TArray<TObjectPtr<UClass>>		DynClassPropArray;

	UPROPERTY()
	TArray<TObjectPtr<APawn>>		DynPawnPropArray;


	UPROPERTY()
	FVector		StructProp;

	UPROPERTY()
	FVector		StructPropArray[2];

	UPROPERTY()
	TArray<FVector> DynStructPropArray;
};

UCLASS()
class UVMTestClassB : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY()
	TObjectPtr<UObject>	BObjectRef;
};

