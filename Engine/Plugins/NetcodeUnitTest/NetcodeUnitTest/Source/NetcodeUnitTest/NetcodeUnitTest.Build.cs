// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NetcodeUnitTest : ModuleRules
	{
		public NetcodeUnitTest(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Sockets"
				}
			);

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"ApplicationCore",
					"EngineSettings",
					"InputCore",
					"PacketHandler",
					"RenderCore",
					"Slate",
					"SlateCore",
					"NetCore"
				}
			);

			if (Target.LinkType != TargetLinkType.Monolithic)
			{
				PrivateDependencyModuleNames.AddRange
				(
					new string[]
					{
						"StandaloneRenderer"
					}
				);
			}

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}

