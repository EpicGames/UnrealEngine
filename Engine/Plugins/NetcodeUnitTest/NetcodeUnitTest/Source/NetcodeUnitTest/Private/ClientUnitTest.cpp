// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientUnitTest.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "GameFramework/Character.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/PlayerController.h"
#include "Engine/ActorChannel.h"


#include "UnitTestManager.h"
#include "MinimalClient.h"
#include "NUTActor.h"
#include "UnitTestEnvironment.h"
#include "NUTGlobals.h"
#include "NUTUtilDebug.h"

#if PLATFORM_WINDOWS
#include "HAL/PlatformNamedPipe.h"
#endif

// @todo #JohnB: Create a unit test for load-testing servers, using multiple instances of the minimal client,
//					as a feature for engine testing and licensees in general (and to flesh-out support for multiple min clients,
//					in the unit test code).
//					Pass this on to QA for testing as a feature, once done.


// @todo #JohnB: IMPORTANT: unit tests need to detect when the wrong net driver is enabled in config,
//					to prevent blocking of unit tests without an intelligible error.

/**
 * UClientUnitTest
 */

UClientUnitTest::UClientUnitTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, UnitTestFlags(EUnitTestFlags::None)
	, MinClientFlags(EMinClientFlags::None)
	, MinClientClass(UMinimalClient::StaticClass())
	, BaseServerURL(TEXT(""))
	, BaseServerParameters(TEXT(""))
	, BaseClientURL(TEXT(""))
	, BaseClientParameters(TEXT(""))
	, AllowedClientActors()
	, AllowedClientRPCs()
	, ServerHandle(nullptr)
	, ServerAddress(TEXT(""))
	, BeaconAddress(TEXT(""))
	, ClientHandle(nullptr)
	, bBlockingServerDelay(false)
	, bBlockingClientDelay(false)
	, bBlockingMinClientDelay(false)
	, NextBlockingTimeout(0.0)
	, MinClient(nullptr)
	, bTriggerredInitialConnect(false)
	, UnitPC(nullptr)
	, bUnitPawnSetup(false)
	, bUnitPlayerStateSetup(false)
	, UnitNUTActor(nullptr)
	, bUnitNUTActorSetup(false)
	, UnitBeacon(nullptr)
	, bReceivedPong(false)
	, bPendingNetworkFailure(false)
{
}

void UClientUnitTest::InitOnlineBeaconClass()
{
	OnlineBeaconClass_Private = FindObject<UClass>(nullptr, TEXT("/Script/OnlineSubsystemUtils.OnlineBeaconClient"));
}

void UClientUnitTest::NotifyAlterMinClient(FMinClientParms& Parms)
{
	UUnitTask* CurTask = UnitTasks.Num() > 0 ? UnitTasks[0] : nullptr;

	if (CurTask != nullptr && !!(CurTask->GetUnitTaskFlags() & EUnitTaskFlags::AlterMinClient))
	{
		CurTask->NotifyAlterMinClient(Parms);
	}
}

void UClientUnitTest::NotifyMinClientConnected()
{
	UnitTaskState |= EUnitTaskFlags::RequireMinClient;

	if (HasAllRequirements())
	{
		ResetTimeout(TEXT("ExecuteClientUnitTest (NotifyMinClientConnected)"));
		ExecuteClientUnitTest();
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePing))
	{
		SendNUTControl(ENUTControlCommand::Ping, TEXT(""));
	}
}

void UClientUnitTest::NotifyControlMessage(FInBunch& Bunch, uint8 MessageType)
{
	if (!!(UnitTestFlags & EUnitTestFlags::DumpControlMessages))
	{
		UNIT_LOG(ELogType::StatusDebug, TEXT("NotifyControlMessage: MessageType: %i (%s), Data Length: %lli (%lli), Raw Data:"), MessageType,
					(FNetControlMessageInfo::IsRegistered(MessageType) ? FNetControlMessageInfo::GetName(MessageType) :
					TEXT("UNKNOWN")), Bunch.GetBytesLeft(), Bunch.GetBitsLeft());

		if (!Bunch.IsError() && Bunch.GetBitsLeft() > 0)
		{
			UNIT_LOG_BEGIN(this, ELogType::StatusDebug | ELogType::StyleMonospace);
			NUTDebug::LogHexDump(Bunch.GetDataPosChecked(), IntCastChecked<int32, int64>(Bunch.GetBytesLeft()), true, true);
			UNIT_LOG_END();
		}
	}
}

void UClientUnitTest::NotifyHandleClientPlayer(APlayerController* PC, UNetConnection* Connection)
{
	UnitPC = PC;

	UnitEnv->UnitTest = this;

	UnitEnv->HandleClientPlayer(UnitTestFlags, PC);

	UnitEnv->UnitTest = nullptr;

	ResetTimeout(TEXT("NotifyHandleClientPlayer"));

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePlayerController) && HasAllRequirements())
	{
		ResetTimeout(TEXT("ExecuteClientUnitTest (NotifyHandleClientPlayer)"));
		ExecuteClientUnitTest();
	}
}

void UClientUnitTest::NotifyAllowNetActor(UClass* ActorClass, bool bActorChannel, bool& bBlockActor)
{
	if (!!(UnitTestFlags & EUnitTestFlags::RequireNUTActor) && ActorClass == ANUTActor::StaticClass() && UnitNUTActor == nullptr)
	{
		bBlockActor = false;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::AcceptPlayerController) && ActorClass->IsChildOf(APlayerController::StaticClass()) &&
		UnitPC == nullptr)
	{
		bBlockActor = false;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePawn) && ActorClass->IsChildOf(ACharacter::StaticClass()) &&
		(!UnitPC.IsValid() || UnitPC->GetCharacter() == nullptr))
	{
		bBlockActor = false;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePlayerState) && ActorClass->IsChildOf(APlayerState::StaticClass()) &&
		(!UnitPC.IsValid() || UnitPC->PlayerState == nullptr))
	{
		bBlockActor = false;
	}

	const UClass* OnlineBeaconClass = GetOnlineBeaconClass();

	check(OnlineBeaconClass != nullptr);

	if (!!(UnitTestFlags & EUnitTestFlags::RequireBeacon) && ActorClass->IsChildOf(OnlineBeaconClass) && UnitBeacon == nullptr)
	{
		bBlockActor = false;
	}

	// @todo #JohnB: Move to minimal client, similar to how AllowClientRPCs was moved? (not needed yet...)
	if (bBlockActor && AllowedClientActors.Num() > 0)
	{
		const auto CheckIsChildOf =
			[&](const UClass* CurEntry)
			{
				return ActorClass->IsChildOf(CurEntry);
			};

		// Use 'ContainsByPredicate' as iterator
		bBlockActor = !AllowedClientActors.ContainsByPredicate(CheckIsChildOf);
	}
}

void UClientUnitTest::NotifyNetActor(UActorChannel* ActorChannel, AActor* Actor)
{
	if (!UnitNUTActor.IsValid())
	{
		// Set this even if not required, as it's needed for some UI elements to function
		UnitNUTActor = Cast<ANUTActor>(Actor);

		if (UnitNUTActor.IsValid())
		{
			// NOTE: ExecuteClientUnitTest triggered for this, in UnitTick - not here.
			ResetTimeout(TEXT("NotifyNetActor - UnitNUTActor"));
		}
	}

	const UClass* OnlineBeaconClass = GetOnlineBeaconClass();

	check(OnlineBeaconClass != nullptr);

	if (!!(UnitTestFlags & EUnitTestFlags::BeaconConnect) && !UnitBeacon.IsValid() && Actor->IsA(OnlineBeaconClass))
	{
		UnitBeacon = Actor;

		NUTNet::HandleBeaconReplicate(UnitBeacon.Get(), MinClient->GetConn());

		if (!!(UnitTestFlags & EUnitTestFlags::RequireBeacon) && HasAllRequirements())
		{
			ResetTimeout(TEXT("ExecuteClientUnitTest (NotifyNetActor - UnitBeacon)"));
			ExecuteClientUnitTest();
		}
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePawn) && !bUnitPawnSetup && UnitPC.IsValid() && Cast<ACharacter>(Actor) != nullptr &&
		UnitPC->GetCharacter() != nullptr)
	{
		bUnitPawnSetup = true;

		ResetTimeout(TEXT("NotifyNetActor - bUnitPawnSetup"));

		if (HasAllRequirements())
		{
			ResetTimeout(TEXT("ExecuteClientUnitTest (NotifyNetActor - bUnitPawnSetup)"));
			ExecuteClientUnitTest();
		}
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePlayerState) && !bUnitPlayerStateSetup && UnitPC.IsValid() &&
		Cast<APlayerState>(Actor) != nullptr && UnitPC->PlayerState != nullptr)
	{
		bUnitPlayerStateSetup = true;

		ResetTimeout(TEXT("NotifyNetActor - bUnitPlayerStateSetup"));

		if (HasAllRequirements())
		{
			ResetTimeout(TEXT("ExecuteClientUnitTest (NotifyNetActor - bUnitPlayerStateSetup)"));
			ExecuteClientUnitTest();
		}
	}
}

void UClientUnitTest::NotifyNetworkFailure(ENetworkFailure::Type FailureType, const FString& ErrorString)
{
	if (!!(UnitTestFlags & EUnitTestFlags::AutoReconnect))
	{
		UNIT_LOG(ELogType::StatusImportant, TEXT("Detected minimal client disconnect when AutoReconnect is enabled. Reconnecting."));

		TriggerAutoReconnect();
	}
	else
	{
		// Only process this error, if a result has not already been returned
		if (VerificationState == EUnitTestVerification::Unverified)
		{
			FString LogMsg = FString::Printf(TEXT("Got network failure of type '%s' (%s)"), ENetworkFailure::ToString(FailureType),
												*ErrorString);

			if (!(UnitTestFlags & EUnitTestFlags::IgnoreDisconnect))
			{
				if (!!(UnitTestFlags & EUnitTestFlags::ExpectDisconnect))
				{
					LogMsg += TEXT(".");

					ELogType CurLogType = GIsAutomationTesting ? ELogType::None : ELogType::StatusWarning;

					UNIT_LOG(CurLogType, TEXT("%s"), *LogMsg);
					UNIT_STATUS_LOG(CurLogType | ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

					bPendingNetworkFailure = true;
				}
				else
				{
					LogMsg += TEXT(", marking unit test as needing update.");

					UNIT_LOG(ELogType::StatusFailure | ELogType::StyleBold, TEXT("%s"), *LogMsg);
					UNIT_STATUS_LOG(ELogType::StatusFailure | ELogType::StatusVerbose | ELogType::StyleBold, TEXT("%s"), *LogMsg);

					VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
				}
			}
			else
			{
				LogMsg += TEXT(".");

				UNIT_LOG(ELogType::StatusWarning, TEXT("%s"), *LogMsg);
				UNIT_STATUS_LOG(ELogType::StatusWarning | ELogType::StatusVerbose, TEXT("%s"), *LogMsg);
			}
		}

		// Shut down the minimal client now (relevant for developer mode)
		if (VerificationState != EUnitTestVerification::Unverified)
		{
			CleanupMinimalClient();
		}
	}
}

void UClientUnitTest::ReceivedControlBunch(FInBunch& Bunch)
{
	if (!Bunch.AtEnd())
	{
		uint8 MessageType = 0;
		Bunch << MessageType;

		if (!Bunch.IsError())
		{
			if (MessageType == NMT_NUTControl)
			{
				ENUTControlCommand CmdType;
				FString Command;

				if (FNetControlMessage<NMT_NUTControl>::Receive(Bunch, CmdType, Command))
				{
					if (!!(UnitTestFlags & EUnitTestFlags::RequirePing) && !bReceivedPong && CmdType == ENUTControlCommand::Pong)
					{
						bReceivedPong = true;

						ResetTimeout(TEXT("ReceivedControlBunch - Ping"));

						if (HasAllRequirements())
						{
							ResetTimeout(TEXT("ExecuteClientUnitTest (ReceivedControlBunch - Ping)"));
							ExecuteClientUnitTest();
						}
					}
					else
					{
						NotifyNUTControl(CmdType, Command);
					}
				}
			}
			else
			{
				NotifyControlMessage(Bunch, MessageType);
			}
		}
	}
}

void UClientUnitTest::NotifyReceiveRPC(AActor* Actor, UFunction* Function, void* Parameters, bool& bBlockRPC)
{
	FString FuncName = Function->GetName();

	// Handle detection and proper setup of the PlayerController's pawn
	if (!!(UnitTestFlags & EUnitTestFlags::RequirePawn) && !bUnitPawnSetup && UnitPC != nullptr)
	{
		if (FuncName == TEXT("ClientRestart"))
		{
			UNIT_LOG(ELogType::StatusImportant, TEXT("Got ClientRestart"));

			// Trigger the event directly here, and block execution in the original code, so we can execute code post-ProcessEvent
			Actor->UObject::ProcessEvent(Function, Parameters);


			// If the pawn is set, now execute the exploit
			if (UnitPC->GetCharacter())
			{
				bUnitPawnSetup = true;

				ResetTimeout(TEXT("bUnitPawnSetup"));

				if (HasAllRequirements())
				{
					ResetTimeout(TEXT("ExecuteClientUnitTest (bUnitPawnSetup)"));
					ExecuteClientUnitTest();
				}
			}
			// If the pawn was not set, get the server to check again
			else
			{
				FString LogMsg = TEXT("Pawn was not set, sending ServerCheckClientPossession request");

				ResetTimeout(LogMsg);
				UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);

				UnitPC->ServerCheckClientPossession();
			}


			bBlockRPC = true;
		}
		// Retries setting the pawn, which will trigger ClientRestart locally, and enters into the above code with the Pawn set
		else if (FuncName == TEXT("ClientRetryClientRestart"))
		{
			bBlockRPC = false;
		}
	}
}

void UClientUnitTest::NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines)
{
	Super::NotifyProcessLog(InProcess, InLogLines);

	// Get partial log messages that indicate startup progress/completion
	const TArray<FString>* ServerStartProgressLogs = nullptr;
	const TArray<FString>* ServerReadyLogs = nullptr;
	const TArray<FString>* ServerTimeoutResetLogs = nullptr;
	const TArray<FString>* ClientTimeoutResetLogs = nullptr;

	UnitEnv->GetServerProgressLogs(ServerStartProgressLogs, ServerReadyLogs, ServerTimeoutResetLogs);
	UnitEnv->GetClientProgressLogs(ClientTimeoutResetLogs);

	// Using 'ContainsByPredicate' as an iterator
	FString MatchedLine;

	const auto SearchInLogLine =
		[&](const FString& ProgressLine)
		{
			bool bFound = false;

			for (auto CurLine : InLogLines)
			{
				if (CurLine.Contains(ProgressLine))
				{
					MatchedLine = CurLine;
					bFound = true;

					break;
				}
			}

			return bFound;
		};


	if (ServerHandle.IsValid() && InProcess.HasSameObject(ServerHandle.Pin().Get()))
	{
		// If launching a server, delay joining by the minimal client until the server has fully setup, and reset the unit test timeout,
		// each time there is a server log event, that indicates progress in starting up
		if (!!(UnitTestFlags & EUnitTestFlags::LaunchServer))
		{
			UNetConnection* UnitConn = (MinClient != nullptr ? MinClient->GetConn() : nullptr);

			if (!bTriggerredInitialConnect && (UnitConn == nullptr || UnitConn->GetConnectionState() == EConnectionState::USOCK_Pending))
			{
				if (ServerReadyLogs->ContainsByPredicate(SearchInLogLine))
				{
					UnitTaskState |= EUnitTaskFlags::RequireServer;

					// Fire off minimal client connection
					if (UnitConn == nullptr)
					{
						FString LogMsg = TEXT("Detected successful server startup, launching minimal client.");
						bool bBlockingProcess = IsBlockingProcessPresent(true);
						bool bBlockingTask = IsTaskBlocking(EUnitTaskFlags::BlockMinClient);

						if (bBlockingProcess)
						{
							LogMsg = TEXT("Detected successful server startup, delaying minimal client due to blocking process.");

							bBlockingMinClientDelay = true;
						}
						else if (bBlockingTask)
						{
							LogMsg = TEXT("Detected successful server startup, delaying minimal client due to blocking task.");
						}

						if (bBlockingTask)
						{
							UnitTaskState |= EUnitTaskFlags::BlockMinClient;
						}

						UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);
						UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

						if (!bBlockingProcess && !bBlockingTask)
						{
							ConnectMinimalClient();
						}
					}

					ResetTimeout(FString(TEXT("ServerReady: ")) + MatchedLine);
				}
				else if (ServerStartProgressLogs->ContainsByPredicate(SearchInLogLine))
				{
					ResetTimeout(FString(TEXT("ServerStartProgress: ")) + MatchedLine);
				}
			}

			if (ServerTimeoutResetLogs->Num() > 0)
			{
				if (ServerTimeoutResetLogs->ContainsByPredicate(SearchInLogLine))
				{
					ResetTimeout(FString(TEXT("ServerTimeoutReset: ")) + MatchedLine, true, 60);
				}
			}
		}
	}

	if (!!(UnitTestFlags & EUnitTestFlags::LaunchClient) && ClientHandle.IsValid() && InProcess.HasSameObject(ClientHandle.Pin().Get()))
	{
		if (ClientTimeoutResetLogs->Num() > 0)
		{
			if (ClientTimeoutResetLogs->ContainsByPredicate(SearchInLogLine))
			{
				ResetTimeout(FString(TEXT("ClientTimeoutReset: ")) + MatchedLine, true, 60);
			}
		}
	}

	// @todo #JohnBLowPri: Consider also, adding a way to communicate with launched clients,
	//				to reset their connection timeout upon server progress, if they fully startup before the server does
}

void UClientUnitTest::NotifyProcessFinished(TWeakPtr<FUnitTestProcess> InProcess)
{
	Super::NotifyProcessFinished(InProcess);

	if (InProcess.IsValid())
	{
		bool bServerFinished = false;
		bool bClientFinished  = false;

		if (ServerHandle.IsValid() && ServerHandle.HasSameObject(InProcess.Pin().Get()))
		{
			bServerFinished = true;
		}
		else if (ClientHandle.IsValid() && ClientHandle.HasSameObject(InProcess.Pin().Get()))
		{
			bClientFinished = true;
		}

		if (bServerFinished || bClientFinished)
		{
			bool bProcessError = false;
			FString UpdateMsg;

			// If the server just finished, cleanup the minimal client
			if (bServerFinished)
			{
				FString LogMsg = TEXT("Server process has finished, cleaning up minimal client.");

				UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);
				UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);


				// Immediately cleanup the minimal client (don't wait for end-of-life cleanup in CleanupUnitTest)
				CleanupMinimalClient();


				// If a server exit was unexpected, mark the unit test as broken
				if (!(UnitTestFlags & EUnitTestFlags::IgnoreServerCrash) && VerificationState == EUnitTestVerification::Unverified)
				{
					UpdateMsg = TEXT("Unexpected server exit, marking unit test as needing update.");
					bProcessError = true;
				}
			}

			// If a client exit was unexpected, mark the unit test as broken
			if (bClientFinished && !(UnitTestFlags & EUnitTestFlags::IgnoreClientCrash) &&
				VerificationState == EUnitTestVerification::Unverified)
			{
				UpdateMsg = TEXT("Unexpected client exit, marking unit test as needing update.");
				bProcessError = true;
			}


			// If either the client/server finished, process the error
			if (bProcessError)
			{
				UNIT_LOG(ELogType::StatusFailure | ELogType::StyleBold, TEXT("%s"), *UpdateMsg);
				UNIT_STATUS_LOG(ELogType::StatusFailure | ELogType::StatusVerbose | ELogType::StyleBold, TEXT("%s"), *UpdateMsg);

				VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
			}
		}
	}
}

void UClientUnitTest::NotifySuspendRequest()
{
#if PLATFORM_WINDOWS
	TSharedPtr<FUnitTestProcess> CurProcess = (ServerHandle.IsValid() ? ServerHandle.Pin() : nullptr);

	if (CurProcess.IsValid())
	{
		// Suspend request
		if (CurProcess->SuspendState == ESuspendState::Active)
		{
			if (SendNUTControl(ENUTControlCommand::SuspendProcess, TEXT("")))
			{
				NotifyProcessSuspendState(ServerHandle, ESuspendState::Suspended);

				UNIT_LOG(, TEXT("Sent suspend request to server (may take time to execute, if server is still starting)."));
			}
			else
			{
				UNIT_LOG(, TEXT("Failed to send suspend request to server"));
			}
		}
		// Resume request
		else if (CurProcess->SuspendState == ESuspendState::Suspended)
		{
			// Send the resume request over a named pipe - this is the only line of communication once suspended
			FString ResumePipeName = FString::Printf(TEXT("%s%i"), NUT_SUSPEND_PIPE, CurProcess->ProcessID);
			FPlatformNamedPipe ResumePipe;

			if (ResumePipe.Create(ResumePipeName, false, false))
			{
				if (ResumePipe.IsReadyForRW())
				{
					int32 ResumeVal = 1;
					ResumePipe.WriteInt32(ResumeVal); //-V530

					UNIT_LOG(, TEXT("Sent resume request to server."));

					NotifyProcessSuspendState(ServerHandle, ESuspendState::Active);
				}
				else
				{
					UNIT_LOG(, TEXT("WARNING: Resume pipe not ready for read/write (server still starting?)."));
				}

				ResumePipe.Destroy();
			}
			else
			{
				UNIT_LOG(, TEXT("Failed to create named pipe, for sending resume request (server still starting?)."));
			}
		}
	}
#else
	UNIT_LOG(ELogType::StatusImportant, TEXT("Suspend/Resume is only supported in Windows."));
#endif
}

void UClientUnitTest::NotifyProcessSuspendState(TWeakPtr<FUnitTestProcess> InProcess, ESuspendState InSuspendState)
{
	Super::NotifyProcessSuspendState(InProcess, InSuspendState);

	if (InProcess == ServerHandle)
	{
		OnSuspendStateChange.ExecuteIfBound(InSuspendState);
	}
}


bool UClientUnitTest::NotifyConsoleCommandRequest(FString CommandContext, FString Command)
{
	bool bHandled = Super::NotifyConsoleCommandRequest(CommandContext, Command);

	if (!bHandled)
	{
		if (CommandContext == TEXT("Local"))
		{
			UNIT_LOG_BEGIN(this, ELogType::OriginConsole);
			bHandled = GEngine->Exec((MinClient != nullptr ? MinClient->GetUnitWorld() : nullptr), *Command, *GLog);
			UNIT_LOG_END();
		}
		else if (CommandContext == TEXT("Server"))
		{
			// @todo #JohnBBug: Perhaps add extra checks here, to be sure we're ready to send console commands?
			//
			//				UPDATE: Yes, this is a good idea, because if the client hasn't gotten to the correct login stage
			//				(NMT_Join or such, need to check when server rejects non-login control commands),
			//				then it leads to an early disconnect when you try to spam-send a command early.
			//
			//				It's easy to test this, just type in a command before join, and hold down enter on the edit box to spam it.

			if (SendNUTControl(ENUTControlCommand::Command_NoResult, Command))
			{
				UNIT_LOG(ELogType::OriginConsole, TEXT("Sent command '%s' to server."), *Command);

				bHandled = true;
			}
			else
			{
				UNIT_LOG(ELogType::OriginConsole, TEXT("Failed to send console command '%s' to server."), *Command);
			}
		}
		else if (CommandContext == TEXT("Client"))
		{
			// @todo #JohnBFeature

			UNIT_LOG(ELogType::OriginConsole, TEXT("Client console commands not yet implemented"));
		}
	}

	return bHandled;
}

void UClientUnitTest::GetCommandContextList(TArray<TSharedPtr<FString>>& OutList, FString& OutDefaultContext)
{
	Super::GetCommandContextList(OutList, OutDefaultContext);

	OutList.Add(MakeShareable(new FString(TEXT("Local"))));

	if (!!(UnitTestFlags & EUnitTestFlags::LaunchServer))
	{
		OutList.Add(MakeShareable(new FString(TEXT("Server"))));
	}

	if (!!(UnitTestFlags & EUnitTestFlags::LaunchClient))
	{
		OutList.Add(MakeShareable(new FString(TEXT("Client"))));
	}

	OutDefaultContext = TEXT("Local");
}


bool UClientUnitTest::SendNUTControl(ENUTControlCommand CommandType, FString Command)
{
	bool bSuccess = false;

	if (MinClient != nullptr)
	{
		FOutBunch* ControlChanBunch = MinClient->CreateChannelBunchByName(NAME_Control, 0);

		if (ControlChanBunch != nullptr)
		{
			uint8 ControlMsg = NMT_NUTControl;
			ENUTControlCommand CmdType = CommandType;

			*ControlChanBunch << ControlMsg;
			*ControlChanBunch << CmdType;
			*ControlChanBunch << Command;

			bSuccess = MinClient->SendControlBunch(ControlChanBunch);
		}
		else
		{
			FString LogMsg = TEXT("Failed to create control channel bunch.");

			UNIT_LOG(ELogType::StatusFailure, TEXT("%s"), *LogMsg);
			UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);
		}
	}

	return bSuccess;
}

bool UClientUnitTest::SendRPCChecked(UObject* Target, const TCHAR* FunctionName, void* Parms, int16 ParmsSize,
										int16 ParmsSizeCorrection/*=0*/)
{
	return MinClient->SendRPCChecked(Target, FunctionName, Parms, ParmsSize, ParmsSizeCorrection);
}

bool UClientUnitTest::SendRPCChecked(UObject* Target, FFuncReflection& FuncRefl)
{
	return MinClient->SendRPCChecked(Target, FuncRefl);
}

bool UClientUnitTest::SendUnitRPCChecked_Internal(UObject* Target, FString RPCName)
{
	bool bSuccess = false;

	MinClient->PreSendRPC();

	if (UnitNUTActor.IsValid())
	{
		UnitNUTActor->ExecuteOnServer(Target, RPCName);
	}
	else
	{
		const TCHAR* LogMsg = TEXT("SendUnitRPCChecked: UnitNUTActor not set.");

		UNIT_LOG(ELogType::StatusFailure, TEXT("%s"), LogMsg);
		UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), LogMsg);
	}

	bSuccess = MinClient->PostSendRPC(RPCName, UnitNUTActor.Get());

	return bSuccess;
}

void UClientUnitTest::OnRPCFailure()
{
	VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
}

void UClientUnitTest::SendGenericExploitFailLog()
{
	SendNUTControl(ENUTControlCommand::Command_NoResult, GetGenericExploitFailLog());
}

bool UClientUnitTest::ValidateUnitTestSettings(bool bCDOCheck/*=false*/)
{
	bool bSuccess = Super::ValidateUnitTestSettings();

	ValidateUnitFlags(UnitTestFlags, MinClientFlags);


	// Validate the rest of the flags which cross-check against non-flag variables

	// If launching a server, make sure the base URL for the server is set
	UNIT_ASSERT(!(UnitTestFlags & EUnitTestFlags::LaunchServer) || BaseServerURL.Len() > 0);

	// If launching a client, make sure some default client parameters have been set (to avoid e.g. launching fullscreen)
	UNIT_ASSERT(!(UnitTestFlags & EUnitTestFlags::LaunchClient) || BaseClientParameters.Len() > 0);


	// You can't specify an allowed actors list, without the AcceptActors flag
	UNIT_ASSERT(AllowedClientActors.Num() == 0 || !!(MinClientFlags & EMinClientFlags::AcceptActors));

#if UE_BUILD_SHIPPING
	// You can't hook ProcessEvent or block RPCs in shipping builds, as the main engine hook is not available in shipping; soft-fail
	if (!!(UnitTestFlags & EUnitTestFlags::NotifyProcessEvent) || !(MinClientFlags & EMinClientFlags::NotifyProcessNetEvent))
	{
		UNIT_LOG(ELogType::StatusFailure | ELogType::StyleBold, TEXT("Unit tests run in shipping mode, can't hook ProcessEvent."));

		bSuccess = false;
	}
#endif

	// If the ping requirements flag is set, it should be the ONLY one set
	//	(which means only one bit should be set, and one bit means it should be power-of-two)
	UNIT_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePing) ||
					FMath::IsPowerOfTwo((uint32)(UnitTestFlags & EUnitTestFlags::RequirementsMask)));

	// If you require a pawn, validate the existence of certain RPC's that are needed for pawn setup and verification
	UNIT_ASSERT(!(UnitTestFlags & EUnitTestFlags::RequirePawn) || (
				GetDefault<APlayerController>()->FindFunction(FName(TEXT("ClientRestart"))) != nullptr &&
				GetDefault<APlayerController>()->FindFunction(FName(TEXT("ClientRetryClientRestart"))) != nullptr));

	// If connecting to a beacon, you must specify the beacon type
	UNIT_ASSERT(!(UnitTestFlags & EUnitTestFlags::BeaconConnect) || ServerBeaconType.Len() > 0);


	// Don't accept any 'Ignore' flags, once the unit test is finalized (they're debug only - all crashes must be handled in final code)
	UNIT_ASSERT(bWorkInProgress || !(UnitTestFlags & (EUnitTestFlags::IgnoreServerCrash | EUnitTestFlags::IgnoreClientCrash |
				EUnitTestFlags::IgnoreDisconnect)));


	return bSuccess;
}

EUnitTestFlags UClientUnitTest::GetMetRequirements()
{
	EUnitTestFlags ReturnVal = EUnitTestFlags::None;

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePlayerController) && UnitPC != nullptr)
	{
		ReturnVal |= EUnitTestFlags::RequirePlayerController;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePawn) && UnitPC != nullptr && UnitPC->GetCharacter() != nullptr && bUnitPawnSetup)
	{
		ReturnVal |= EUnitTestFlags::RequirePawn;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePlayerState) && UnitPC != nullptr && UnitPC->PlayerState != nullptr &&
		bUnitPlayerStateSetup)
	{
		ReturnVal |= EUnitTestFlags::RequirePlayerState;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequirePing) && bReceivedPong)
	{
		ReturnVal |= EUnitTestFlags::RequirePing;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequireNUTActor) && UnitNUTActor.IsValid() && bUnitNUTActorSetup)
	{
		ReturnVal |= EUnitTestFlags::RequireNUTActor;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequireBeacon) && UnitBeacon.IsValid())
	{
		ReturnVal |= EUnitTestFlags::RequireBeacon;
	}

	// ExecuteClientUnitTest should be triggered manually - unless you override HasAllCustomRequirements
	if (!!(UnitTestFlags & EUnitTestFlags::RequireCustom) && HasAllCustomRequirements())
	{
		ReturnVal |= EUnitTestFlags::RequireCustom;
	}

	return ReturnVal;
}

bool UClientUnitTest::HasAllRequirements(bool bIgnoreCustom/*=false*/)
{
	bool bReturnVal = true;

	// The minimal client creation/connection is now delayed, so need to wait for that too
	if (MinClient == nullptr || !MinClient->IsConnected())
	{
		bReturnVal = false;
	}

	EUnitTestFlags RequiredFlags = (UnitTestFlags & EUnitTestFlags::RequirementsMask);

	if (bIgnoreCustom)
	{
		RequiredFlags &= ~EUnitTestFlags::RequireCustom;
	}

	if ((RequiredFlags & GetMetRequirements()) != RequiredFlags)
	{
		bReturnVal = false;
	}

	return bReturnVal;
}

ELogType UClientUnitTest::GetExpectedLogTypes()
{
	ELogType ReturnVal = Super::GetExpectedLogTypes();

	if (!!(UnitTestFlags & EUnitTestFlags::LaunchServer))
	{
		ReturnVal |= ELogType::Server;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::LaunchClient))
	{
		ReturnVal |= ELogType::Client;
	}

	if (!!(UnitTestFlags & EUnitTestFlags::DumpControlMessages))
	{
		ReturnVal |= ELogType::StatusDebug;
	}

	if (!!(MinClientFlags & (EMinClientFlags::DumpReceivedRaw | EMinClientFlags::DumpSendRaw)))
	{
		ReturnVal |= ELogType::StatusDebug;
	}

	return ReturnVal;
}

void UClientUnitTest::ResetTimeout(FString ResetReason, bool bResetConnTimeout/*=false*/, uint32 MinDuration/*=0*/)
{
	// Extend the timeout to at least two minutes, if a crash is expected, as sometimes crash dumps take a very long time
	if (!!(UnitTestFlags & EUnitTestFlags::ExpectServerCrash) &&
			(ResetReason.Contains("ExecuteClientUnitTest") || ResetReason.Contains("Detected crash.")))
	{
		MinDuration = FMath::Max<uint32>(MinDuration, 120);
		bResetConnTimeout = true;
	}

	Super::ResetTimeout(ResetReason, bResetConnTimeout, MinDuration);

	if (bResetConnTimeout && MinClient != nullptr)
	{
		MinClient->ResetConnTimeout((float)(FMath::Max(MinDuration, UnitTestTimeout)));
	}
}


bool UClientUnitTest::ExecuteUnitTest()
{
	bool bSuccess = ValidateUnitTestSettings();

	if (bSuccess)
	{
		if (!!(UnitTestFlags & EUnitTestFlags::LaunchServer))
		{
			bool bBlockingProcess = IsBlockingProcessPresent(true);

			if (bBlockingProcess)
			{
				FString LogMsg = TEXT("Delaying server startup due to blocking process");

				UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);
				UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

				bBlockingServerDelay = true;
			}
			else
			{
				StartUnitTestServer();
			}

			if (!!(UnitTestFlags & EUnitTestFlags::LaunchClient))
			{
				if (bBlockingProcess)
				{
					FString LogMsg = TEXT("Delaying client startup due to blocking process");

					UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);
					UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

					bBlockingClientDelay = true;
				}
				else
				{
					// Client handle is set outside of StartUnitTestClient, in case support for multiple clients is added later
					ClientHandle = StartUnitTestClient(ServerAddress);
				}
			}
		}
		// Immediately execute
		else
		{
			ResetTimeout(TEXT("ExecuteClientUnitTest (ExecuteUnitTest - No server)"));
			ExecuteClientUnitTest();
		}
	}
	else
	{
		FString LogMsg = TEXT("Failed to validate unit test settings/environment");

		UNIT_LOG(ELogType::StatusFailure, TEXT("%s"), *LogMsg);
		UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);
	}

	return bSuccess;
}

void UClientUnitTest::CleanupUnitTest(EUnitTestResetStage ResetStage)
{
	if (ResetStage <= EUnitTestResetStage::ResetConnection)
	{
		if (MinClient != nullptr)
		{
			FProcessEventHook::Get().RemoveEventHook(MinClient->GetUnitWorld());
		}

		CleanupMinimalClient();
	}

	if (ResetStage != EUnitTestResetStage::None)
	{
		if (ResetStage <= EUnitTestResetStage::FullReset)
		{
			ServerHandle.Reset();
			ServerAddress = TEXT("");
			BeaconAddress = TEXT("");
			ClientHandle.Reset();
			bBlockingServerDelay = false;
			bBlockingClientDelay = false;
			bTriggerredInitialConnect = false;
		}

		if (ResetStage <= EUnitTestResetStage::ResetConnection)
		{
			bBlockingMinClientDelay = false;
			NextBlockingTimeout = 0.0;
		}
	}

	Super::CleanupUnitTest(ResetStage);
}

bool UClientUnitTest::ConnectMinimalClient(const TCHAR* InNetID/*=nullptr*/)
{
	bool bSuccess = false;
	FMinClientHooks Hooks;

	check(MinClient == nullptr);

	Hooks.ConnectedDel = FOnMinClientConnected::CreateUObject(this, &UClientUnitTest::NotifyMinClientConnected);
	Hooks.NetworkFailureDel = FOnMinClientNetworkFailure::CreateUObject(this, &UClientUnitTest::NotifyNetworkFailure);
#if !UE_BUILD_SHIPPING
	Hooks.SendRPCDel = FOnSendRPC::CreateUObject(this, &UClientUnitTest::NotifySendRPC);
#endif
	Hooks.ReceivedControlBunchDel = FOnMinClientReceivedControlBunch::CreateUObject(this, &UClientUnitTest::ReceivedControlBunch);
	Hooks.RepActorSpawnDel = FOnMinClientRepActorSpawn::CreateUObject(this, &UClientUnitTest::NotifyAllowNetActor);
	Hooks.HandleClientPlayerDel = FOnHandleClientPlayer::CreateUObject(this, &UClientUnitTest::NotifyHandleClientPlayer);

	if (!!(UnitTestFlags & EUnitTestFlags::CaptureReceivedRaw))
	{
		Hooks.ReceivedRawPacketDel = FOnMinClientReceivedRawPacket::CreateUObject(this, &UClientUnitTest::NotifyReceivedRawPacket);
	}

#if !UE_BUILD_SHIPPING
	Hooks.LowLevelSendDel.BindUObject(this, &UClientUnitTest::NotifySocketSendRawPacket);
#endif


	EMinClientFlags CurMinClientFlags = FromUnitTestFlags(UnitTestFlags) | MinClientFlags;

	if (!!(CurMinClientFlags & EMinClientFlags::NotifyNetActors))
	{
		Hooks.NetActorDel.BindUObject(this, &UClientUnitTest::NotifyNetActor);
	}


	Hooks.ReceiveRPCDel.BindUObject(this, &UClientUnitTest::NotifyReceiveRPC);
	Hooks.RPCFailureDel.BindUObject(this, &UClientUnitTest::OnRPCFailure);


	FMinClientParms Parms;

	Parms.MinClientFlags = CurMinClientFlags;
	Parms.ServerAddress = ServerAddress;
	Parms.BeaconAddress = BeaconAddress;
	Parms.BeaconType = ServerBeaconType;
	Parms.Timeout = UnitTestTimeout;

	Parms.AllowedClientRPCs = AllowedClientRPCs;

	NotifyAlterMinClient(Parms);

	if (InNetID != nullptr)
	{
		Parms.JoinUID = InNetID;
	}

	MinClient = NewObject<UMinimalClient>(GetTransientPackage(), MinClientClass);

	MinClient->SetInterface(this);

	bSuccess = MinClient->Connect(Parms, Hooks);

	if (bSuccess)
	{
		if (!!(UnitTestFlags & EUnitTestFlags::NotifyProcessEvent))
		{
#if !UE_BUILD_SHIPPING
			FProcessEventHook::Get().AddEventHook(MinClient->GetUnitWorld(),
				FOnProcessNetEvent::CreateUObject(this, &UClientUnitTest::NotifyProcessEvent));
#else
			FString LogMsg = TEXT("Require ProcessEvent hook, but current build configuration does not support it.");

			UNIT_LOG(ELogType::StatusFailure, TEXT("%s"), *LogMsg);
			UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

			bSuccess = false;
#endif
		}

		bTriggerredInitialConnect = true;
	}
	else
	{
		FString LogMsg = TEXT("Failed to connect minimal client.");

		UNIT_LOG(ELogType::StatusFailure, TEXT("%s"), *LogMsg);
		UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);
	}

	return bSuccess;
}

void UClientUnitTest::CleanupMinimalClient()
{
	if (MinClient != nullptr)
	{
		MinClient->Cleanup();
		MinClient = nullptr;
	}

	UnitPC = nullptr;
	bUnitPawnSetup = false;
	bUnitPlayerStateSetup = false;
	UnitNUTActor = nullptr;
	bUnitNUTActorSetup = false;
	UnitBeacon = nullptr;
	bReceivedPong = false;
	bPendingNetworkFailure = false;
}

void UClientUnitTest::TriggerAutoReconnect()
{
	UNIT_LOG(ELogType::StatusImportant, TEXT("Performing Auto-Reconnect."))

	CleanupMinimalClient();
	ConnectMinimalClient();
}


void UClientUnitTest::StartUnitTestServer()
{
	if (!ServerHandle.IsValid())
	{
		FString LogMsg = TEXT("Unit test launching a server");

		UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);
		UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

		// Determine the new server ports
		int32 ServerPort = 0;
		int32 ServerBeaconPort = 0;

		GetNextServerPorts(ServerPort, ServerBeaconPort);


		// Setup the launch URL
		FString ServerParameters = ConstructServerParameters() + FString::Printf(TEXT(" -Port=%i"), ServerPort);

		if (!!(UnitTestFlags & EUnitTestFlags::BeaconConnect))
		{
			ServerParameters += FString::Printf(TEXT(" -BeaconPort=%i -NUTMonitorBeacon"), ServerBeaconPort);
		}

		ServerHandle = StartUEUnitTestProcess(ServerParameters, true, EBuildTargetType::Server);

		if (ServerHandle.IsValid())
		{
			ServerAddress = FString::Printf(TEXT("127.0.0.1:%i"), ServerPort);

			if (!!(UnitTestFlags & EUnitTestFlags::BeaconConnect))
			{
				BeaconAddress = FString::Printf(TEXT("127.0.0.1:%i"), ServerBeaconPort);
			}
			else
			{
				int32 FoundBeaconPort = 0;

				// Detect BeaconPort values injected into the server commandline, by the UnitTestEnvironment
				if (FParse::Value(*ServerParameters, TEXT("BeaconPort="), FoundBeaconPort) && FoundBeaconPort != 0)
				{
					BeaconAddress = FString::Printf(TEXT("127.0.0.1:%i"), FoundBeaconPort);
				}
			}

			TSharedPtr<FUnitTestProcess> CurHandle = ServerHandle.Pin();

			CurHandle->ProcessTag = FString::Printf(TEXT("UE_Server_%i"), CurHandle->ProcessID);
			CurHandle->BaseLogType = ELogType::Server;
			CurHandle->LogPrefix = TEXT("[SERVER]");
			CurHandle->MainLogColor = COLOR_CYAN;
			CurHandle->SlateLogColor = FLinearColor(0.f, 1.f, 1.f);

			// Strip out 'Error: ' from server log output, when expecting a crash during automation testing,
			// in order to avoid triggering an automation test failure
			if (GIsAutomationTesting && !!(UnitTestFlags & EUnitTestFlags::ExpectServerCrash))
			{
				CurHandle->bStripErrorLogs = true;
			}
		}
	}
	else
	{
		UNIT_LOG(ELogType::StatusFailure, TEXT("ERROR: Server process already started."));
	}
}

FString UClientUnitTest::ConstructServerParameters()
{
	// Construct the server log parameter
	FString GameLogDir = FPaths::ProjectLogDir();
	FString ServerLogParam;

	if (!UnitLogDir.IsEmpty() && UnitLogDir.StartsWith(GameLogDir))
	{
		ServerLogParam = TEXT(" -Log=") + UnitLogDir.Mid(GameLogDir.Len()) + TEXT("UnitTestServer.log");
	}
	else
	{
		ServerLogParam = TEXT(" -Log=UnitTestServer.log");
	}

	// NOTE: In the absence of "-ddc=noshared", a VPN connection can cause UE to take a long time to startup
	// NOTE: Without '-CrashForUAT'/'-unattended' the auto-reporter can pop up
	// NOTE: Without '-UseAutoReporter' the crash report executable is launched
	// NOTE: Without '?bIsLanMatch', the Steam net driver will be active, when OnlineSubsystemSteam is in use
	FString Parameters = FPaths::GetProjectFilePath() + TEXT(" ") + BaseServerURL + TEXT("?bIsLanMatch") + TEXT(" -server ") +
							BaseServerParameters + ServerLogParam +
							TEXT(" -stdout -FullStdOutLogOutput -ddc=noshared -unattended -CrashForUAT -UseAutoReporter -NUTServer");
		
							// @todo #JohnB: Remove eventually, or wrap with CL #if's, based on when FullStdOutLogOutput was added
							//TEXT(" -AllowStdOutLogVerbosity")
							;

							// Removed this, to support detection of shader compilation, based on shader compiler .exe
							//TEXT(" -NoShaderWorker");

	// Only append -ForceLogFlush, if it's present on this processes commandline
	if (FParse::Param(FCommandLine::Get(), TEXT("ForceLogFlush")))
	{
		Parameters += TEXT(" -ForceLogFlush");
	}

	return Parameters;
}

void UClientUnitTest::GetNextServerPorts(int32& OutServerPort, int32& OutBeaconPort, bool bAdvance/*=true*/)
{
	// Determine the new server port
	int32 DefaultPort = 0;
	GConfig->GetInt(TEXT("URL"), TEXT("Port"), DefaultPort, GEngineIni);

	// Increment the server port used by 10, for every unit test
	int32& ServerPortOffset = UNUTGlobals::Get().ServerPortOffset;
	int32 CurPortOffset = ServerPortOffset + 10;

	if (bAdvance)
	{
		ServerPortOffset = CurPortOffset;
	}

	OutServerPort = DefaultPort + 50 + CurPortOffset;
	OutBeaconPort = OutServerPort + 5;
}

TWeakPtr<FUnitTestProcess> UClientUnitTest::StartUnitTestClient(FString ConnectIP, bool bMinimized/*=true*/)
{
	TWeakPtr<FUnitTestProcess> ReturnVal = nullptr;

	FString LogMsg = TEXT("Unit test launching a client");

	UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *LogMsg);
	UNIT_STATUS_LOG(ELogType::StatusVerbose, TEXT("%s"), *LogMsg);

	FString ClientParameters = ConstructClientParameters(ConnectIP);

	ReturnVal = StartUEUnitTestProcess(ClientParameters, bMinimized);

	if (ReturnVal.IsValid())
	{
		auto CurHandle = ReturnVal.Pin();

		// @todo #JohnBMultiClient: If you add support for multiple clients, make the log prefix numbered, also try to differentiate colours
		CurHandle->ProcessTag = FString::Printf(TEXT("UE_Client_%i"), CurHandle->ProcessID);
		CurHandle->BaseLogType = ELogType::Client;
		CurHandle->LogPrefix = TEXT("[CLIENT]");
		CurHandle->MainLogColor = COLOR_GREEN;
		CurHandle->SlateLogColor = FLinearColor(0.f, 1.f, 0.f);
	}

	return ReturnVal;
}

FString UClientUnitTest::ConstructClientParameters(FString ConnectIP)
{
	// Construct the client log parameter
	FString GameLogDir = FPaths::ProjectLogDir();
	FString ClientLogParam;

	if (!UnitLogDir.IsEmpty() && UnitLogDir.StartsWith(GameLogDir))
	{
		ClientLogParam = TEXT(" -Log=") + UnitLogDir.Mid(GameLogDir.Len()) + TEXT("UnitTestClient.log");
	}
	else
	{
		ClientLogParam = TEXT(" -Log=UnitTestClient.log");
	}

	// NOTE: In the absence of "-ddc=noshared", a VPN connection can cause UE to take a long time to startup
	// NOTE: Without '-CrashForUAT'/'-unattended' the auto-reporter can pop up
	// NOTE: Without '-UseAutoReporter' the crash report executable is launched
	FString Parameters = FPaths::GetProjectFilePath() + TEXT(" ") + ConnectIP + BaseClientURL + TEXT(" -game ") + BaseClientParameters +
							ClientLogParam + TEXT(" -stdout -AllowStdOutLogVerbosity -ddc=noshared -nosplash -unattended") +
							TEXT(" -CrashForUAT -nosound -UseAutoReporter");

							// Removed this, to support detection of shader compilation, based on shader compiler .exe
							//TEXT(" -NoShaderWorker")

	// Only append -ForceLogFlush, if it's present on this processes commandline
	if (FParse::Param(FCommandLine::Get(), TEXT("ForceLogFlush")))
	{
		Parameters += TEXT(" -ForceLogFlush");
	}

	return Parameters;
}

void UClientUnitTest::PrintUnitTestProcessErrors(TSharedPtr<FUnitTestProcess> InHandle)
{
	// If this was the server, and we were not expecting a crash, print out a warning
	if (!(UnitTestFlags & EUnitTestFlags::ExpectServerCrash) && ServerHandle.IsValid() && InHandle == ServerHandle.Pin())
	{
		FString LogMsg = TEXT("WARNING: Got server crash, but unit test not marked as expecting a server crash.");

		STATUS_SET_COLOR(FLinearColor(1.0, 1.0, 0.0));

		UNIT_LOG(ELogType::StatusWarning, TEXT("%s"), *LogMsg);
		UNIT_STATUS_LOG(ELogType::StatusWarning, TEXT("%s"), *LogMsg);

		STATUS_RESET_COLOR();
	}

	Super::PrintUnitTestProcessErrors(InHandle);
}

void UClientUnitTest::UnitTick(float DeltaTime)
{
	if (bBlockingServerDelay || bBlockingClientDelay || bBlockingMinClientDelay)
	{
		bool bBlockingProcess = IsBlockingProcessPresent();

		if (!bBlockingProcess)
		{
			ResetTimeout(TEXT("Blocking Process Reset"), true, 60);

			auto IsWaitingOnTimeout =
				[&]()
				{
					return NextBlockingTimeout > FPlatformTime::Seconds();
				};

			if (bBlockingServerDelay && !IsWaitingOnTimeout())
			{
				StartUnitTestServer();

				bBlockingServerDelay = false;
				NextBlockingTimeout = FPlatformTime::Seconds() + 10.0;
			}

			if (bBlockingClientDelay && !IsWaitingOnTimeout())
			{
				ClientHandle = StartUnitTestClient(ServerAddress);

				bBlockingClientDelay = false;
				NextBlockingTimeout = FPlatformTime::Seconds() + 10.0;
			}

			if (bBlockingMinClientDelay && !IsWaitingOnTimeout())
			{
				if (!IsTaskBlocking(EUnitTaskFlags::BlockMinClient))
				{
					ConnectMinimalClient();
				}

				bBlockingMinClientDelay = false;
				NextBlockingTimeout = FPlatformTime::Seconds() + 10.0;
			}
		}
	}

	if (MinClient != nullptr)
	{
		if (MinClient->IsTickable())
		{
			MinClient->UnitTick(DeltaTime);
		}

		// Prevent net connection timeout in developer mode
		if (bDeveloperMode)
		{
			MinClient->ResetConnTimeout(120.f);
		}
	}

	if (!!(UnitTestFlags & EUnitTestFlags::RequireNUTActor) && !bUnitNUTActorSetup && UnitNUTActor.IsValid() &&
		(!!(UnitTestFlags & EUnitTestFlags::RequireBeacon) || UnitNUTActor->GetOwner() != nullptr))
	{
		bUnitNUTActorSetup = true;

		if (HasAllRequirements())
		{
			ResetTimeout(TEXT("ExecuteClientUnitTest (bUnitNUTActorSetup)"));
			ExecuteClientUnitTest();
		}
	}

	Super::UnitTick(DeltaTime);

	// After there has been a chance to process remaining server output, finish handling the pending disconnect
	if (VerificationState == EUnitTestVerification::Unverified && bPendingNetworkFailure)
	{
		const TCHAR* LogMsg = TEXT("Handling pending disconnect, marking unit test as needing update.");

		UNIT_LOG(ELogType::StatusFailure | ELogType::StyleBold, TEXT("%s"), LogMsg);
		UNIT_STATUS_LOG(ELogType::StatusFailure | ELogType::StatusVerbose | ELogType::StyleBold, TEXT("%s"), LogMsg);

		VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;

		bPendingNetworkFailure = false;
	}
}

bool UClientUnitTest::IsTickable() const
{
	bool bReturnVal = Super::IsTickable();

	bReturnVal = bReturnVal || bDeveloperMode || bBlockingServerDelay || bBlockingClientDelay || bBlockingMinClientDelay ||
					(MinClient != nullptr && MinClient->IsTickable()) || bPendingNetworkFailure;

	return bReturnVal;
}

void UClientUnitTest::LogComplete()
{
	Super::LogComplete();

	if (!HasAllRequirements())
	{
		EUnitTestFlags UnmetRequirements = EUnitTestFlags::RequirementsMask & UnitTestFlags & ~(GetMetRequirements());
		EUnitTestFlags CurRequirement = (EUnitTestFlags)1;
		FString UnmetStr;

		while (UnmetRequirements != EUnitTestFlags::None)
		{
			if (!!(CurRequirement & UnmetRequirements))
			{
				if (UnmetStr != TEXT(""))
				{
					UnmetStr += TEXT(", ");
				}

				UnmetStr += GetUnitTestFlagName(CurRequirement);

				UnmetRequirements &= ~CurRequirement;
			}

			CurRequirement = (EUnitTestFlags)((uint32)CurRequirement << 1);
		}

		UNIT_LOG(ELogType::StatusFailure, TEXT("Failed to meet unit test requirements: %s"), *UnmetStr);
	}
}

void UClientUnitTest::UnblockEvents(EUnitTaskFlags ReadyEvents)
{
	Super::UnblockEvents(ReadyEvents);

	if (!!(ReadyEvents & EUnitTaskFlags::BlockMinClient))
	{
		if (!bBlockingMinClientDelay)
		{
			ConnectMinimalClient();
		}
	}
}

bool UClientUnitTest::IsConnectionLogSource(UNetConnection* InConnection)
{
	bool bReturnVal = Super::IsConnectionLogSource(InConnection);

	if (!bReturnVal && MinClient != nullptr && MinClient->GetConn() == InConnection)
	{
		bReturnVal = true;
	}

	return bReturnVal;
}

