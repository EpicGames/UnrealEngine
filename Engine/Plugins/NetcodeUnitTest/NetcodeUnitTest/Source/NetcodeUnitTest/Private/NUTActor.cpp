// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTActor.h"
#include "Misc/CommandLine.h"
#include "Misc/StringOutputDevice.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/GameMode.h"
#include "GameFramework/GameStateBase.h"
#include "GameFramework/PlayerState.h"
#include "Engine/LocalPlayer.h"
#include "UnrealEngine.h"

#if PLATFORM_WINDOWS
#include "HAL/PlatformNamedPipe.h"
#endif

// @todo #JohnBDoc: Need to tidy up and fully document this class; not all of the code below is clear

#include "Net/NUTUtilNet.h"
#include "NUTGlobals.h"
#include "NUTUtil.h"

const FName NAME_BeaconDriver = FName(TEXT("BeaconNetDriver"));

IMPLEMENT_CONTROL_CHANNEL_MESSAGE(NUTControl);


ANUTActor::ANUTActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, BeaconDriverName(NAME_None)
	, LastAliveTime(0.f)
	, bMonitorForBeacon(false)
{
	PrimaryActorTick.bCanEverTick = true;

	bReplicates = true;
	bAlwaysRelevant = true;
	SetReplicatingMovement(false);
	SetNetUpdateFrequency(1);
}

void ANUTActor::PostActorCreated()
{
	Super::PostActorCreated();

	// Hook the net driver notify, to capture custom control channel messages
	UWorld* CurWorld = GetWorld();

	if (CurWorld != NULL)
	{	
		HookNetDriver(NUTUtil::GetActiveNetDriver(CurWorld));
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("NUTMonitorBeacon")))
	{
		bMonitorForBeacon = true;
	}
}

UNetConnection* ANUTActor::GetNetConnection() const
{
	UNetConnection* ReturnVal = Super::GetNetConnection();

	// If no net connection is found (happens when connected to a beacon, due to not having an Owner), autodetect the correct connection
	if (ReturnVal == NULL)
	{
		UNetDriver* NetDriver = GetNetDriver();
		UNetConnection* ServerConn = (NetDriver != NULL ? ToRawPtr(NetDriver->ServerConnection) : NULL);

		if (ServerConn != NULL)
		{
			ReturnVal = ServerConn;
		}
		// If this is the server, set based on beacon driver client connection.
		// Only the server has a net driver of name NAME_BeaconDriver (clientside the name is chosen dynamically)
		else if (NetDriver != NULL && NetDriver->NetDriverName == BeaconDriverName && NetDriver->ClientConnections.Num() > 0)
		{
			ReturnVal = NetDriver->ClientConnections[0];
		}
	}

	return ReturnVal;
}

bool ANUTActor::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch)
{
	bool bHandledMessage = false;

	if (MessageType == NMT_NUTControl)
	{
		// Some commands won't work without an owner, so if one is not set, set it now
		if (Cast<APlayerController>(GetOwner()) == NULL)
		{
			UpdateOwner();
		}

		ENUTControlCommand CmdType;
		FString Command;

		if (FNetControlMessage<NMT_NUTControl>::Receive(Bunch, CmdType, Command))
		{
			// Console command
			if (CmdType == ENUTControlCommand::Command_NoResult || CmdType == ENUTControlCommand::Command_SendResult)
			{
				UE_LOGF(LogUnitTest, Log, "NMT_NUTControl: Executing command: %ls", *Command);

				FStringOutputDevice CmdResult;
				CmdResult.SetAutoEmitLineTerminator(true);

				bool bCmdSuccess = false;

				bCmdSuccess = GEngine->Exec(GetWorld(), *Command, CmdResult);

				UE_LOGF(LogUnitTest, Log, "NMT_NUTControl: Command result: %ls", *CmdResult);


				bool bSendResult = CmdType == ENUTControlCommand::Command_SendResult;

				if (bSendResult)
				{
					ENUTControlCommand ReturnCmdType = (bCmdSuccess ? ENUTControlCommand::CommandResult_Success :
						ENUTControlCommand::CommandResult_Failed);

					FNetControlMessage<NMT_NUTControl>::Send(Connection, ReturnCmdType, CmdResult);
				}
			}
			// Console command result
			else if (CmdType == ENUTControlCommand::CommandResult_Failed || CmdType == ENUTControlCommand::CommandResult_Success)
			{
				bool bCmdSuccess = CmdType == ENUTControlCommand::CommandResult_Success;

				if (bCmdSuccess)
				{
					UE_LOGF(LogUnitTest, Log, "NMT_NUTControl: Got command result:");
					UE_LOGF(LogUnitTest, Log, "%ls", *Command);
				}
				else
				{
					UE_LOGF(LogUnitTest, Log, "NMT_NUTControl: Failed to execute command");
				}
			}
			// Ping request
			else if (CmdType == ENUTControlCommand::Ping)
			{
				ENUTControlCommand TempCmdType = ENUTControlCommand::Pong;
				FString Dud;

				FNetControlMessage<NMT_NUTControl>::Send(Connection, TempCmdType, Dud);
			}
			// Pong reply - this should only be implemented by custom unit tests; hence the assert
			else if (CmdType == ENUTControlCommand::Pong)
			{
				UNIT_ASSERT(false);
			}
			// Custom implemented events, with the result triggered through 'NotifyEvent'
			else if (CmdType == ENUTControlCommand::WatchEvent)
			{
				// NOTE: Only the last NetConnection to request a WatchEvent, will receive notifications
				UNUTGlobals::Get().EventWatcher = Connection;

				// Watch for the end of seamless travel
				if (Command == TEXT("SeamlessTravelEnd"))
				{
					FCoreUObjectDelegates::PostLoadMapWithWorld.AddStatic(&ANUTActor::NotifyPostLoadMap);
				}
			}
			// Event watch notification - should only be implemented by custom unit tests
			else if (CmdType == ENUTControlCommand::NotifyEvent)
			{
				UNIT_ASSERT(false);
			}
			// Create an actor instance (the 'summon' console command, doesn't work without a cheat manager)
			else if (CmdType == ENUTControlCommand::Summon)
			{
				const TCHAR* Cmd = *Command;
				FString SpawnClassName = FParse::Token(Cmd, false);
				bool bForceBeginPlay = FParse::Param(Cmd, TEXT("ForceBeginPlay"));

				// Hack specifically for getting the GameplayDebugger working - think the mainline code is broken
				bool bGameplayDebuggerHack = FParse::Param(Cmd, TEXT("GameplayDebuggerHack"));

				UClass* SpawnClass = FindObject<UClass>(NULL, *SpawnClassName);

				if (SpawnClass != nullptr)
				{
					FActorSpawnParameters SpawnParms;
					SpawnParms.Owner = GetOwner();

					AActor* NewActor = GetWorld()->SpawnActor<AActor>(SpawnClass, SpawnParms);

					if (NewActor != nullptr)
					{
						UE_LOGF(LogUnitTest, Log, "Successfully summoned actor of class '%ls'", *SpawnClassName);

						if (bForceBeginPlay && !NewActor->HasActorBegunPlay())
						{
							UE_LOGF(LogUnitTest, Log, "Forcing call to 'BeginPlay' on newly spawned actor.");

							NewActor->DispatchBeginPlay();
						}

						if (bGameplayDebuggerHack)
						{
							// Assign LocalPlayerOwner property, to the PC owning this NUTActor, using reflection (to avoid dependency)
							FObjectProperty* LocalPlayerOwnerProp =
								FindFProperty<FObjectProperty>(NewActor->GetClass(), TEXT("LocalPlayerOwner"));

							if (LocalPlayerOwnerProp != NULL)
							{
								LocalPlayerOwnerProp->SetObjectPropertyValue(
									LocalPlayerOwnerProp->ContainerPtrToValuePtr<UObject*>(NewActor), GetOwner());
							}
							else
							{
								UE_LOGF(LogUnitTest, Log, "WARNING: Failed to find 'LocalPlayerOwner'. Unit test broken.");
							}

							// Also hack-disable ticking, so that the replicator doesn't spawn a second replicator
							NewActor->SetActorTickEnabled(false);
						}
					}
					else
					{
						UE_LOGF(LogUnitTest, Log, "SpawnActor failed for class '%ls'", *Command);
					}
				}
				else
				{
					UE_LOGF(LogUnitTest, Log, "Could not find actor class '%ls'", *Command);
				}
			}
			// Suspend the game, until a resume request is received (used for giving time, to attach a debugger)
			else if (CmdType == ENUTControlCommand::SuspendProcess)
			{
#if PLATFORM_WINDOWS
				UE_LOGF(LogUnitTest, Log, "Suspend start.");


				// Setup a named pipe, to monitor for the resume request
				FString ResumePipeName = FString::Printf(TEXT("%s%u"), NUT_SUSPEND_PIPE, FPlatformProcess::GetCurrentProcessId());
				FPlatformNamedPipe ResumePipe;
				bool bPipeCreated = ResumePipe.Create(ResumePipeName, true, false);

				if (bPipeCreated)
				{
					if (!ResumePipe.OpenConnection())
					{
						UE_LOGF(LogUnitTest, Log, "WARNING: Failed to open pipe connection.");
					}
				}
				else
				{
					UE_LOGF(LogUnitTest, Log, "WARNING: Failed to create resume pipe.");
				}



				// Spin/sleep (effectively suspended) until a resume request is received
				while (true)
				{
					if (bPipeCreated && ResumePipe.IsReadyForRW())
					{
						int32 ResumeVal = 0;

						if (ResumePipe.ReadInt32(ResumeVal) && !!ResumeVal)
						{
							UE_LOGF(LogUnitTest, Log, "Got resume request.");
							break;
						}
					}


					FPlatformProcess::Sleep(1.0f);
				}


				ResumePipe.Destroy();

				UE_LOGF(LogUnitTest, Log, "Suspend end.");
#else
				UE_LOGF(LogUnitTest, Log, "Suspend/Resume is only supported in Windows.");
#endif
			}
		}

		bHandledMessage = true;
	}

	return bHandledMessage;
}

void ANUTActor::NotifyPostLoadMap(UWorld* LoadedWorld)
{
	if (VerifyEventWatcher())
	{
		ENUTControlCommand CmdType = ENUTControlCommand::NotifyEvent;
		FString Command = TEXT("NotifySeamlessTravelEnd");

		FNetControlMessage<NMT_NUTControl>::Send(UNUTGlobals::Get().EventWatcher, CmdType, Command);
	}
}

// Safety check, to ensure the EventWatcher UNetConnection is still valid
bool ANUTActor::VerifyEventWatcher()
{
	bool bVerified = false;
	TObjectPtr<UNetConnection>& EventWatcher = UNUTGlobals::Get().EventWatcher;

	if (EventWatcher)
	{
		UWorld* CurWorld = NUTUtil::GetPrimaryWorld();
		UNetDriver* CurDriver = (CurWorld != nullptr ? NUTUtil::GetActiveNetDriver(CurWorld) : nullptr);

		if (CurDriver != nullptr && CurDriver->ClientConnections.Contains(EventWatcher))
		{
			bVerified = true;
		}
		else
		{
			EventWatcher = nullptr;
			bVerified = false;
		}
	}

	return bVerified;
}

void ANUTActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	UWorld* CurWorld = GetWorld();

	if (CurWorld != nullptr)
	{
		ENetMode CurNetMode = GEngine != nullptr ? GEngine->GetNetMode(CurWorld) : NM_Standalone;
		bool bClientTimedOut = CurNetMode != NM_Standalone &&
								(CurWorld->RealTimeSeconds - LastAliveTime) > (CurNetMode == NM_Client ? 5.0f : 10.0f);

		// Have the client tell the server they are still alive
		if (CurNetMode == NM_Client && bClientTimedOut)
		{
			ServerClientStillAlive();
			LastAliveTime = CurWorld->RealTimeSeconds;
		}

		// Have the server set the owner, when appropriate
		if (Cast<APlayerController>(GetOwner()) == nullptr || bClientTimedOut)
		{
			UpdateOwner();
		}


		// Monitor for the beacon net driver, so it can be hooked
		if (bMonitorForBeacon)
		{
			// Somehow, the beacon driver name got messed up in a subsequent checkin, so now has to be found manually
			UNetDriver* BeaconDriver = nullptr;

			FWorldContext* CurContext = GEngine->GetWorldContextFromWorld(CurWorld);

			if (CurContext != nullptr)
			{
				for (const FNamedNetDriver& CurDriverRef : CurContext->ActiveNetDrivers)
				{
					if (CurDriverRef.NetDriverDef->DefName == NAME_BeaconDriver)
					{
						BeaconDriver = CurDriverRef.NetDriver;
						break;
					}
				}
			}

			// Only hook when a client is connected
			if (BeaconDriver != nullptr && BeaconDriver->ClientConnections.Num() > 0)
			{
				HookNetDriver(BeaconDriver);

				UE_LOGF(LogUnitTest, Log, "Hooked beacon net driver");


				// Also switch over replication to the beacon net driver
				SetReplicates(false);

				BeaconDriverName = BeaconDriver->NetDriverName;
				NetDriverName = BeaconDriverName;

				SetReplicates(true);


				// Send an RPC, for force actor channel replication
				NetMulticastPing();

				bMonitorForBeacon = false;
			}
		}
	}
}

void ANUTActor::UpdateOwner()
{
	UWorld* CurWorld = GetWorld();
	ENetMode CurNetMode = GEngine != NULL ? GEngine->GetNetMode(CurWorld) : NM_Standalone;
	AGameStateBase* GameState = CurWorld->GetGameState();

	if (GameState != NULL && GEngine != NULL && CurNetMode != NM_Client)
	{
		// @todo #JohnBReview: You want this to only happen if no remote players are present (perhaps give all players control instead,
		//				if this becomes a problem - requires setting things up differently though)
		if (CurNetMode == NM_ListenServer || CurNetMode == NM_Standalone)
		{
			for (FLocalPlayerIterator It(GEngine, CurWorld); It; ++It)
			{
				if (It->PlayerController != NULL)
				{
					// Reset LastAliveTime, to give client a chance to send initial 'alive' RPC
					LastAliveTime = CurWorld->RealTimeSeconds;

					SetOwner(It->PlayerController);
					break;
				}
			}
		}

		for (int i=0; i<GameState->PlayerArray.Num(); i++)
		{
			APlayerController* PC = Cast<APlayerController>(GameState->PlayerArray[i] != NULL ?
															GameState->PlayerArray[i]->GetOwner() : NULL);

			if (PC != NULL && PC != GetOwner() && Cast<UNetConnection>(PC->Player) != NULL)
			{
				UE_LOGF(LogUnitTest, Log, "Setting NUTActor owner to: %ls (%ls)", *PC->GetName(),
					*GameState->PlayerArray[i]->GetPlayerName());

				// Reset LastAliveTime, to give client a chance to send initial 'alive' RPC
				LastAliveTime = CurWorld->RealTimeSeconds;

				SetOwner(PC);

				break;
			}
		}
	}
}

void ANUTActor::HookNetDriver(UNetDriver* TargetNetDriver)
{
	if (TargetNetDriver != NULL)
	{
		FNetworkNotifyHook* NewNotify = new FNetworkNotifyHook(TargetNetDriver->Notify);
		TargetNetDriver->Notify = NewNotify;

		NewNotify->NotifyControlMessageDelegate.BindUObject(this, &ANUTActor::NotifyControlMessage);


		// If a custom net driver timeout was specified on the commandline, apply it
		int32 CustomTimeout = 0;

		if (FParse::Value(FCommandLine::Get(), TEXT("NUTConnectionTimeout="), CustomTimeout))
		{
			UE_LOGF(LogUnitTest, Log, "Setting %ls InitialConnectTimeout/ConnectionTimeout to '%i'", *TargetNetDriver->GetFullName(),
					CustomTimeout);

			TargetNetDriver->InitialConnectTimeout = static_cast<float>(CustomTimeout);
			TargetNetDriver->ConnectionTimeout = static_cast<float>(CustomTimeout);
		}
	}
}

bool ANUTActor::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bReturnVal = false;

	// Only execute for actual live instances
	if (GetClass()->GetDefaultObject() != this)
	{
		bReturnVal = ProcessConsoleExec(Cmd, Ar, NULL);
	}

	return bReturnVal;
}


void ANUTActor::Admin(FString Command)
{
	if (!Command.IsEmpty())
	{
		ServerAdmin(Command);
	}
}

bool ANUTActor::ServerAdmin_Validate(const FString& Command)
{
	return true;
}

void ANUTActor::ServerAdmin_Implementation(const FString& Command)
{
	UE_LOGF(LogUnitTest, Log, "Executing command: %ls", *Command);

	GEngine->Exec(GetWorld(), *Command);
}


void ANUTActor::UnitSeamlessTravel(FString Dest/*=TEXT(" ")*/)
{
	AGameMode* GameMode = (GetWorld() != NULL ? GetWorld()->GetAuthGameMode<AGameMode>() : NULL);

	if (GameMode != NULL)
	{
		UE_LOGF(LogUnitTest, Log, "Executing seamless travel");

		bool bOldUseSeamlessTravel = GameMode->bUseSeamlessTravel;
		GameMode->bUseSeamlessTravel = true;

		if (Dest.TrimStartAndEnd().IsEmpty())
		{
			GetWorld()->ServerTravel(TEXT("?restart"));
		}
		else
		{
			GetWorld()->ServerTravel(Dest);
		}

		GameMode->bUseSeamlessTravel = bOldUseSeamlessTravel;
	}
}

void ANUTActor::UnitTravel(FString Dest/*=TEXT(" ")*/)
{
	AGameMode* GameMode = (GetWorld() != NULL ? GetWorld()->GetAuthGameMode<AGameMode>() : NULL);

	if (GameMode != NULL)
	{
		UE_LOGF(LogUnitTest, Log, "Executing normal travel");

		bool bOldUseSeamlessTravel = GameMode->bUseSeamlessTravel;
		GameMode->bUseSeamlessTravel = false;

		if (Dest.TrimStartAndEnd().IsEmpty())
		{
			GetWorld()->ServerTravel(TEXT("?restart"));
		} 
		else
		{
			GetWorld()->ServerTravel(Dest);
		}

		GameMode->bUseSeamlessTravel = bOldUseSeamlessTravel;
	}
}

void ANUTActor::NetFlush()
{
	UNetDriver* CurNetDriver = GetNetDriver();

	if (CurNetDriver != nullptr)
	{
		UNetConnection* ServerConn = CurNetDriver->ServerConnection;

		if (ServerConn != nullptr)
		{
			UE_LOGF(LogUnitTest, Log, "Flushing ServerConnection");
			ServerConn->FlushNet();
		}
		else
		{
			UE_LOGF(LogUnitTest, Log, "Flushing ClientConnections");

			for (UNetConnection* CurConn : CurNetDriver->ClientConnections)
			{
				CurConn->FlushNet();
			}
		}
	}
}
void ANUTActor::Wait(uint16 Seconds)
{
	if (Seconds > 0)
	{
		UE_LOGF(LogUnitTest, Log, "Sleeping for '%i' seconds", Seconds);

		FPlatformProcess::Sleep(Seconds);
	}
	else
	{
		UE_LOGF(LogUnitTest, Log, "Bad 'Wait' command value '%i'", Seconds);
	}
}


bool ANUTActor::ServerClientStillAlive_Validate()
{
	return true;
}

void ANUTActor::ServerClientStillAlive_Implementation()
{
	LastAliveTime = GetWorld()->RealTimeSeconds;
}

bool ANUTActor::ServerReceiveText_Validate(const FText& InText)
{
	return true;
}

void ANUTActor::ServerReceiveText_Implementation(const FText& InText)
{
	UE_LOGF(LogUnitTest, Log, "ServerReceiveText: InText: %ls", *InText.ToString());
}

bool ANUTActor::ServerClientPing_Validate()
{
	return true;
}

void ANUTActor::ServerClientPing_Implementation()
{
	// If any clients have not loaded the current level, do nothing
	bool bNotLoaded = false;

	UWorld* CurWorld = GetWorld();
	UNetDriver* CurNetDriver = (CurWorld != nullptr ? NUTUtil::GetActiveNetDriver(CurWorld) : nullptr);

	if (CurNetDriver != nullptr)
	{
		for (UNetConnection* CurConn : CurNetDriver->ClientConnections)
		{
			// Based on UNetDriver::IsLevelInitializeForActor
			bNotLoaded = !(CurConn->GetClientWorldPackageName() == CurWorld->GetOutermost()->GetFName() &&
										CurConn->ClientHasInitializedLevel(GetLevel()));

			// Also trigger if there is no PlayerController yet set for the connection
			if (CurConn->OwningActor == nullptr)
			{
				bNotLoaded = true;
			}


			if (bNotLoaded)
			{
				break;
			}
		}
	}

	if (!bNotLoaded)
	{
		NetMulticastPing();
	}
}

void ANUTActor::NetMulticastPing_Implementation()
{
	UWorld* CurWorld = GetWorld();

	if (CurWorld->GetNetMode() == NM_Client && !NUTNet::IsUnitTestWorld(CurWorld))
	{
		UE_LOGF(LogUnitTest, Log, "Unit Test Client Ping.");
	}
}

void ANUTActor::ExecuteOnServer(UObject* InTargetObj, FString InTargetFunc)
{
	if (InTargetObj != nullptr && InTargetFunc.Len() > 0)
	{
		// Only static functions can be used, so verify this is referencing a static function
		FName TargetFuncName = *InTargetFunc;
		UFunction* TargetFuncObj = InTargetObj->FindFunction(TargetFuncName);

		if (FString(TargetFuncName.ToString()).StartsWith(TEXT("UnitTestServer_"), ESearchCase::CaseSensitive))
		{
			if (TargetFuncObj != nullptr)
			{
				if (TargetFuncObj->HasAnyFunctionFlags(FUNC_Static))
				{
					UObject* TargetObjCDO = (InTargetObj->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject) ?
											InTargetObj :
											InTargetObj->GetClass()->GetDefaultObject());

					// Now that we've verified it's a static function, change the delegate object to the class default object
					// (as that is where static function must be executed, as there is no serverside unit test instance),
					// and then send it to the server
					TempDelegate.BindUFunction(TargetObjCDO, TargetFuncName);

					FDelegateProperty* DelProp = FindFProperty<FDelegateProperty>(GetClass(), TEXT("TempDelegate"));

					FString DelString;
					DelProp->ExportTextItem_InContainer(DelString, this, nullptr, this, 0, nullptr);

					ServerExecute(DelString);
				}
				else
				{
					UE_LOGF(LogUnitTest, Log, "ExecuteOnServer: Only static functions can be passed to the server.");
				}
			}
			else
			{
				UE_LOGF(LogUnitTest, Log, "ExecuteOnServer: Could not locate InTarget function.");
			}
		}
		else
		{
			UE_LOGF(LogUnitTest, Log, "ExecuteOnServer: Target functions must be prefixed 'UnitTestServer_FuncName'");
		}
	}
	else
	{
		UE_LOGF(LogUnitTest, Log, "ExecuteOnServer: Target not specified");
	}
}

bool ANUTActor::ServerExecute_Validate(const FString& InDelegate)
{
	return InDelegate.Len() > 0;
}

void ANUTActor::ServerExecute_Implementation(const FString& InDelegate)
{
	// Convert the string back into a delegate, and execute
	FDelegateProperty* DelProp = FindFProperty<FDelegateProperty>(GetClass(), TEXT("TempDelegate"));

	const TCHAR* InDelText = *InDelegate;

	TempDelegate.Unbind();
	DelProp->ImportText_InContainer(InDelText, this, NULL, 0);

	if (TempDelegate.IsBound())
	{
		UE_LOGF(LogUnitTest, Log, "Executing serverside unit test function '%ls::%ls'",
				*TempDelegate.GetUObject()->GetClass()->GetName(), *TempDelegate.GetFunctionName().ToString());

		TempDelegate.Execute(this);
	}
	else
	{
		UE_LOGF(LogUnitTest, Log, "ServerExecute: Failed to find function '%ls'", InDelText);
	}
}
