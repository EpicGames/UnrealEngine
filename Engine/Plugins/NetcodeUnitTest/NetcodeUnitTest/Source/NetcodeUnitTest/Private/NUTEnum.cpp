// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTEnum.h"

/**
 * EMinClientFlags
 */

EMinClientFlags FromUnitTestFlags(EUnitTestFlags Flags)
{
	EMinClientFlags ReturnFlags = EMinClientFlags::None;

	#define CHECK_FLAG(x) ReturnFlags |= !!(Flags & EUnitTestFlags::x) ? EMinClientFlags::x : EMinClientFlags::None

	CHECK_FLAG(BeaconConnect);

	#undef CHECK_FLAG

	return ReturnFlags;
}


/**
 * EUnitTestFlags
 */

FString GetUnitTestFlagName(EUnitTestFlags Flag)
{
	#define EUTF_CASE(x) case EUnitTestFlags::x : return TEXT(#x)

	switch (Flag)
	{
		EUTF_CASE(None);
		EUTF_CASE(LaunchServer);
		EUTF_CASE(LaunchClient);
		EUTF_CASE(AcceptPlayerController);
		EUTF_CASE(BeaconConnect);
		EUTF_CASE(RequirePlayerController);
		EUTF_CASE(RequirePawn);
		EUTF_CASE(RequirePlayerState);
		EUTF_CASE(RequirePing);
		EUTF_CASE(RequireNUTActor);
		EUTF_CASE(RequireBeacon);
		EUTF_CASE(RequireCustom);
		EUTF_CASE(ExpectServerCrash);
		EUTF_CASE(ExpectDisconnect);
		EUTF_CASE(IgnoreServerCrash);
		EUTF_CASE(IgnoreClientCrash);
		EUTF_CASE(IgnoreDisconnect);
		EUTF_CASE(NotifyProcessEvent);
		EUTF_CASE(CaptureReceivedRaw);
		EUTF_CASE(DumpControlMessages);


	default:
		if (FMath::IsPowerOfTwo((uint32)Flag))
		{
			return FString::Printf(TEXT("Unknown 0x%08X"), (uint32)Flag);
		}
		else
		{
			return FString::Printf(TEXT("Bad/Multiple flags 0x%08X"), (uint32) Flag);
		}
	}

	#undef EUTF_CASE
}

