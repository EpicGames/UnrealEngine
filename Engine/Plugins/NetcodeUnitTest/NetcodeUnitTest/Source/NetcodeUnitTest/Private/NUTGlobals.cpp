// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTGlobals.h"


/**
 * UNUTGlobals
 */

UNUTGlobals::UNUTGlobals(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EventWatcher(nullptr)
	, ServerPortOffset(0)
	, UnitTestNetDriverCount(0)
	, DumpRPCMatches()
	, UnitTestModules()
	, UnloadedModules()
{
}

