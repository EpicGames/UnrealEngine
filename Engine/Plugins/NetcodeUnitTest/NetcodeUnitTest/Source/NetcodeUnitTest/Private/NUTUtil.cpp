// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTUtil.h"


#include "UnitTest.h"
#include "Misc/OutputDeviceFile.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"



// Globals

static FAssertHookDevice GAssertHook;


/**
 * FAssertHookDevice
 */

void FAssertHookDevice::AddAssertHook(FString Assert)
{
	// Hook GError when an assert hook is first added
	if (GError != &GAssertHook)
	{
		GAssertHook.HookDevice(GError);

		GError = &GAssertHook;
	}

	GAssertHook.DisabledAsserts.Add(Assert);
}


/**
 * NUTUtil
 */

void NUTUtil::GetUnitTestClassDefList(TArray<UUnitTest*>& OutUnitTestClassDefaults)
{
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(UUnitTest::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract))
		{
			UUnitTest* CurDefault = Cast<UUnitTest>(It->GetDefaultObject());

			if (CurDefault != nullptr)
			{
				OutUnitTestClassDefaults.Add(CurDefault);
			}
		}
	}
}

void NUTUtil::SortUnitTestClassDefList(TArray<UUnitTest*>& InUnitTestClassDefaults)
{
	// @todo #JohnBRefactorLambda: Convert these inline sort functions to lambda's now
	struct FUnitTestDateSort
	{
		FORCEINLINE bool operator()(const UUnitTest& A, const UUnitTest& B) const
		{
			return (A.GetUnitTestDate() < B.GetUnitTestDate());
		}
	};

	struct FUnitTestTypeDateSort
	{
		FUnitTestTypeDateSort(TArray<FString>& InTypeOrder)
			: TypeOrder(InTypeOrder)
		{
		}

		FORCEINLINE bool operator()(const UUnitTest& A, const UUnitTest& B) const
		{
			bool bReturnVal = false;

			if (TypeOrder.IndexOfByKey(A.GetUnitTestType()) < TypeOrder.IndexOfByKey(B.GetUnitTestType()))
			{
				bReturnVal = true;
			}
			else if (TypeOrder.IndexOfByKey(A.GetUnitTestType()) == TypeOrder.IndexOfByKey(B.GetUnitTestType()) &&
						A.GetUnitTestDate() < B.GetUnitTestDate())
			{
				bReturnVal = true;
			}

			return bReturnVal;
		}

		/** The order with which the prioritize types */
		TArray<FString> TypeOrder;
	};


	// First reorder the unit test classes by date, then grab the unit test types by date, then group them by type/date
	TArray<FString> ListTypes;

	InUnitTestClassDefaults.Sort(FUnitTestDateSort());

	for (int i=0; i<InUnitTestClassDefaults.Num(); i++)
	{
		ListTypes.AddUnique(InUnitTestClassDefaults[i]->GetUnitTestType());
	}

	// Now sort again, based both on type and date
	InUnitTestClassDefaults.Sort(FUnitTestTypeDateSort(ListTypes));
}

bool NUTUtil::ParseEnum(const TCHAR* Stream, const TCHAR* Match, const TCHAR* EnumName, uint64& Value)
{
	bool bReturnVal = false;
	FString StrValue;

	if (FParse::Value(Stream, Match, StrValue) && StrValue.Len() > 0)
	{
		UEnum* EnumObj = UClass::TryFindTypeSlow<UEnum>(EnumName);

		if (EnumObj != nullptr)
		{
			int32 EnumCount = EnumObj->NumEnums();
			FString Prefix = (EnumCount > 1 ? EnumObj->GetNameStringByIndex(0) : TEXT(""));

			for (int32 i=1; i<EnumCount && Prefix.Len() > 0; i++)
			{
				FString CurEnum = EnumObj->GetNameStringByIndex(i);
				int32 PrefixIdx = 0;

				while (PrefixIdx < Prefix.Len() && PrefixIdx < CurEnum.Len() && Prefix[PrefixIdx] == CurEnum[PrefixIdx])
				{
					PrefixIdx++;
				}

				Prefix.LeftInline(PrefixIdx);
			}


			if (Prefix.Len() > 0)
			{
				int32 UnderscoreIdx = Prefix.Find(TEXT("_"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);

				if (UnderscoreIdx != INDEX_NONE)
				{
					Prefix.LeftInline(UnderscoreIdx+1);
				}
				else
				{
					Prefix.Reset();
				}

				if (Prefix.Len() > 0 && !StrValue.StartsWith(Prefix))
				{
					StrValue = Prefix + StrValue;
				}
			}


			for (int32 i=0; i<EnumCount; i++)
			{
				FString CurEnum = EnumObj->GetNameStringByIndex(i);

				if (CurEnum == StrValue)
				{
					Value = EnumObj->GetValueByIndex(i);
					bReturnVal = true;

					break;
				}
			}
		}
	}

	return bReturnVal;
}

void NUTUtil::SpecialLog(FOutputDeviceFile* Ar, const TCHAR* SpecialCategory, const TCHAR* Data, ELogVerbosity::Type Verbosity,
							const FName& Category)
{
	bool bOldEmitTerminator = Ar->GetAutoEmitLineTerminator();
	bool bOldSuppressEvent = Ar->GetSuppressEventTag();

	Ar->SetAutoEmitLineTerminator(false);

	// Log the timestamp, special category and verbosity/category tag (uses some log system hacks to achieve clean logs)
	FString SerializeStr = SpecialCategory;

	if (!bOldSuppressEvent)
	{
		if (Category != NAME_None)
		{
			SerializeStr += Category.ToString() + TEXT(":");
		}

		if (Verbosity != ELogVerbosity::Log)
		{
			SerializeStr += ToString(Verbosity);
			SerializeStr += TEXT(": ");
		}
		else if (Category != NAME_None)
		{
			SerializeStr += TEXT(" ");
		}
	}

	Ar->Serialize(*SerializeStr, ELogVerbosity::Log, NAME_None);

	Ar->SetAutoEmitLineTerminator(bOldEmitTerminator);
	Ar->SetSuppressEventTag(true);

	Ar->Serialize(Data, Verbosity, Category);

	Ar->SetSuppressEventTag(bOldSuppressEvent);
}

FString NUTUtil::GetPackageModule(UPackage* Package, bool bVerifyModule/*=true*/)
{
	FString ReturnVal = TEXT("");

	if (Package != nullptr)
	{
		ReturnVal = Package->GetName();

		int32 SlashIdx = ReturnVal.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);

		ReturnVal = (SlashIdx != INDEX_NONE ? ReturnVal.Mid(SlashIdx+1) : TEXT(""));

		if (bVerifyModule && FModuleManager::Get().GetModule(*ReturnVal) == nullptr)
		{
			ReturnVal = TEXT("");
		}
	}

	return ReturnVal;
}


