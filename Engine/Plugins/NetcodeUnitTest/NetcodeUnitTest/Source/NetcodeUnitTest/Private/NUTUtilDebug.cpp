// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTUtilDebug.h"

#include "Misc/OutputDeviceNull.h"

#include "ClientUnitTest.h"
#include "MinimalClient.h"
#include "NUTActor.h"




/**
 * Globals
 */

FStackTraceManager* GTraceManager = new FStackTraceManager();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FLogStackTraceManager* GLogTraceManager = new FLogStackTraceManager();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

namespace UE::NUT
{
	FLogHookManager* GLogHookManager = new FLogHookManager();
	FLogTraceManager* GLogTrace = new FLogTraceManager();
	FLogCommandManager* GLogCommandManager = new FLogCommandManager();
}


bool GGlobalExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return GEngine != nullptr ? GEngine->Exec(InWorld, Cmd, Ar) : false;
}


/**
 * FScopedLog
 */

void FScopedLog::InternalConstruct(const TArray<FString>& InLogCategories, UClientUnitTest* InUnitTest, bool bInRemoteLogging)
{
	LogCategories = InLogCategories;
	UnitTest = InUnitTest;
	bRemoteLogging = bInRemoteLogging;


	UMinimalClient* MinClient = (UnitTest != nullptr ? UnitTest->MinClient.Get() : nullptr);
	UNetConnection* UnitConn = (MinClient != nullptr ? MinClient->GetConn() : nullptr);

	// If you want to do remote logging, you MUST specify the client unit test doing the logging
	if (bRemoteLogging)
	{
		UNIT_ASSERT(UnitTest != nullptr);
		UNIT_ASSERT(UnitConn != nullptr);
	}

	// Flush all current packets, so the log messages only relate to scoped code
	if (UnitConn != nullptr)
	{
		UnitConn->FlushNet();
	}


	const TCHAR* TargetVerbosity = bSuppressLogging ? TEXT("None") : TEXT("All");

	// If specified, enable logs remotely
	if (bRemoteLogging && MinClient != nullptr)
	{
		FOutBunch* ControlChanBunch = MinClient->CreateChannelBunchByName(NAME_Control, 0);

		if (ControlChanBunch != nullptr)
		{
			uint8 ControlMsg = NMT_NUTControl;
			ENUTControlCommand ControlCmd = ENUTControlCommand::Command_NoResult;
			FString Cmd = TEXT("");

			for (const FString& CurCategory : LogCategories)
			{
				Cmd = TEXT("Log ") + CurCategory + TEXT(" ") + TargetVerbosity;

				*ControlChanBunch << ControlMsg;
				*ControlChanBunch << ControlCmd;
				*ControlChanBunch << Cmd;
			}

			MinClient->SendControlBunch(ControlChanBunch);


			// Need to flush again now to get the above parsed on server first
			UnitConn->FlushNet();
		}
	}


	// Now enable local logging
	FString Cmd = TEXT("");
	UWorld* UnitWorld = (MinClient != nullptr ? MinClient->GetUnitWorld() : nullptr);
	FOutputDeviceNull NullAr;

	for (FString CurCategory : LogCategories)
	{
		Cmd = TEXT("Log ") + CurCategory + TEXT(" ") + TargetVerbosity;

		GEngine->Exec(UnitWorld, *Cmd, NullAr);
	}
}

FScopedLog::~FScopedLog()
{
	UMinimalClient* MinClient = (UnitTest != nullptr ? UnitTest->MinClient.Get() : nullptr);
	UNetConnection* UnitConn = (MinClient != nullptr ? MinClient->GetConn() : nullptr);

	// Flush all built-up packets
	if (UnitConn != nullptr)
	{
		UnitConn->FlushNet();
	}


	// Reset local logging
	FString Cmd = TEXT("");
	UWorld* UnitWorld = (MinClient != nullptr ? MinClient->GetUnitWorld() : nullptr);
	FOutputDeviceNull NullAr;

	for (int32 i=LogCategories.Num()-1; i>=0; i--)
	{
		Cmd = TEXT("Log ") + LogCategories[i] + TEXT(" Default");

		GEngine->Exec(UnitWorld, *Cmd, NullAr);
	}


	// Reset remote logging (and flush immediately)
	if (bRemoteLogging && MinClient != nullptr)
	{
		FOutBunch* ControlChanBunch = MinClient->CreateChannelBunchByName(NAME_Control, 0);

		if (ControlChanBunch != nullptr)
		{
			uint8 ControlMsg = NMT_NUTControl;
			ENUTControlCommand ControlCmd = ENUTControlCommand::Command_NoResult;

			for (int32 i=LogCategories.Num()-1; i>=0; i--)
			{
				Cmd = TEXT("Log ") + LogCategories[i] + TEXT(" Default");

				*ControlChanBunch << ControlMsg;
				*ControlChanBunch << ControlCmd;
				*ControlChanBunch << Cmd;
			}

			MinClient->SendControlBunch(ControlChanBunch);

			UnitConn->FlushNet();
		}
	}
}

// @todo #JohnB: Reimplement eventually - see header file, needs to be merged with a similar class
#if 0
#if !UE_BUILD_SHIPPING
/**
 * FProcessEventHookBase
 */

FProcessEventHookBase::FProcessEventHookBase()
	: OrigEventHook()
{
	if (AActor::ProcessEventDelegate.IsBound())
	{
		OrigEventHook = AActor::ProcessEventDelegate;
	}

	AActor::ProcessEventDelegate.BindRaw(this, &FProcessEventHookBase::InternalProcessEventHook);
}

FProcessEventHookBase::~FProcessEventHookBase()
{
	AActor::ProcessEventDelegate = OrigEventHook;
	OrigEventHook.Unbind();
}

bool FProcessEventHookBase::InternalProcessEventHook(AActor* Actor, UFunction* Function, void* Parameters)
{
	bool bReturnVal = false;

	// If there was originally already a ProcessEvent hook in place, transparently pass on the event, so it's not disrupted
	if (OrigEventHook.IsBound())
	{
		bReturnVal = OrigEventHook.Execute(Actor, Function, Parameters);
	}

	ProcessEventHook(Actor, Function, Parameters);

	return bReturnVal;
}
#endif
#endif

// @todo #JohnB: Reimplement when needed. See header file.
#if 0
void FScopedProcessEventLog::ProcessEventHook(AActor* Actor, UFunction* Function, void* Parameters)
{
	UE_LOGF(LogUnitTest, Log, "FScopedProcessEventLog: Actor: %ls, Function: %ls",
			(Actor != nullptr ? *Actor->GetName() : TEXT("nullptr")),
			*Function->GetName());
}
#endif


/**
 * FNUTStackTrace
 */

FNUTStackTrace::FNUTStackTrace(FString InTraceName)
	: TraceName(InTraceName)
	, Tracker()
{
	Tracker.ResetTracking();
}

FNUTStackTrace::~FNUTStackTrace()
{
	Tracker.ResetTracking();
}
void FNUTStackTrace::Enable()
{
	if (!IsTrackingEnabled())
	{
		Tracker.ToggleTracking();
	}
}

void FNUTStackTrace::Disable()
{
	if (IsTrackingEnabled())
	{
		Tracker.ToggleTracking();
	}
}

void FNUTStackTrace::AddTrace(bool bLogAdd/*=false*/)
{
	if (IsTrackingEnabled())
	{
		if (bLogAdd)
		{
			UE_LOGF(LogUnitTest, Log, "Adding stack trace for TraceName '%ls'.", *TraceName);
		}

		Tracker.CaptureStackTrace(TRACE_IGNORE_DEPTH + IgnoreDepthOffset);
	}
}

void FNUTStackTrace::Dump(bool bKeepTraceHistory/*=false*/)
{
	UE_LOGF(LogUnitTest, Log, "Dumping tracked stack traces for TraceName '%ls':", *TraceName);

	Tracker.DumpStackTraces(0, *GLog);

	if (!bKeepTraceHistory)
	{
		Tracker.ResetTracking();
	}
}


/**
 * FStackTraceManager
 */

FStackTraceManager::FStackTraceManager()
	: Traces()
{
}

FStackTraceManager::~FStackTraceManager()
{
	for (TMap<FString, FNUTStackTrace*>::TConstIterator It=Traces.CreateConstIterator(); It; ++It)
	{
		FNUTStackTrace* Trace = It->Value;

		if (Trace != nullptr)
		{
			delete Trace;
		}
	}

	Traces.Empty();
}

void FStackTraceManager::Enable(FString TraceName)
{
	FNUTStackTrace* Trace = GetOrCreateTrace(TraceName);

	Trace->Enable();
}

void FStackTraceManager::Disable(FString TraceName)
{
	FNUTStackTrace* Trace = GetTrace(TraceName);

	if (Trace != nullptr)
	{
		Trace->Disable();
	}
	else
	{
		UE_LOGF(LogUnitTest, Log, "Trace disable: No trace tracking found for TraceName '%ls'.", *TraceName);
	}
}

void FStackTraceManager::AddTrace(FString TraceName, bool bLogAdd/*=false*/, bool bDump/*=false*/, bool bStartDisabled/*=false*/)
{
	bool bIsNewTrace = false;
	FNUTStackTrace* Trace = GetOrCreateTrace(TraceName, &bIsNewTrace);

	if (bIsNewTrace)
	{
		if (bStartDisabled)
		{
			Trace->Disable();
		}
		else
		{
			Trace->Enable();
		}
	}

	if (Trace->IsTrackingEnabled())
	{
		Trace->AddTrace(bLogAdd);

		if (bDump)
		{
			Trace->Dump(true);
		}
	}
}

void FStackTraceManager::Dump(FString TraceName, bool bKeepTraceHistory/*=false*/, bool bKeepTracking/*=true*/)
{
	FNUTStackTrace* Trace = GetTrace(TraceName);

	if (Trace != nullptr)
	{
		Trace->Dump(bKeepTraceHistory);

		if (!bKeepTracking)
		{
			delete Trace;
			Traces.FindAndRemoveChecked(TraceName);
		}
	}
	else
	{
		UE_LOGF(LogUnitTest, Log, "No trace tracking found for TraceName '%ls'.", *TraceName);
	}
}

void FStackTraceManager::Clear(FString TraceName)
{
	FNUTStackTrace* Trace = GetTrace(TraceName);

	if (Trace != nullptr)
	{
		delete Trace;
		Traces.FindAndRemoveChecked(TraceName);
	}
	else
	{
		UE_LOGF(LogUnitTest, Log, "No trace tracking found for TraceName '%ls'.", *TraceName);
	}
}

void FStackTraceManager::DumpAll(bool bKeepTraceHistory/*=false*/, bool bKeepTracking/*=true*/)
{
	UE_LOGF(LogUnitTest, Log, "Dumping all tracked stack traces:");

	for (TMap<FString, FNUTStackTrace*>::TIterator It=Traces.CreateIterator(); It; ++It)
	{
		FNUTStackTrace* Trace = It->Value;

		if (Trace != nullptr)
		{
			Trace->Dump(bKeepTraceHistory);

			if (!bKeepTracking)
			{
				delete Trace;
				It.RemoveCurrent();
			}
		}
	}
}

void FStackTraceManager::TraceAndDump(FString TraceName)
{
	FNUTStackTrace* Trace = GetTrace(TraceName);

	if (Trace == nullptr || Trace->IsTrackingEnabled())
	{
		UE_LOGF(LogUnitTest, Log, "Dumping once-off stack trace for TraceName '%ls':", *TraceName);

		FStackTracker TempTracker(nullptr, nullptr, nullptr, true);

		TempTracker.CaptureStackTrace(TRACE_IGNORE_DEPTH + IgnoreDepthOffset);
		TempTracker.DumpStackTraces(0, *GLog);
		TempTracker.ResetTracking();
	}
}


namespace UE::NUT
{

/**
 * FScopedIncrementTraceIgnoreDepth
 */

FScopedIncrementTraceIgnoreDepth::FScopedIncrementTraceIgnoreDepth(FStackTraceManager* InManager, int32 InIgnoreIncrement)
	: Manager(InManager)
	, IgnoreIncrement(InIgnoreIncrement)
{
	if (Manager != nullptr)
	{
		Manager->IgnoreDepthOffset += IgnoreIncrement;

		for (TMap<FString, FNUTStackTrace*>::TConstIterator It(Manager->Traces); It; ++It)
		{
			It.Value()->IgnoreDepthOffset += IgnoreIncrement;
		}
	}
}

FScopedIncrementTraceIgnoreDepth::~FScopedIncrementTraceIgnoreDepth()
{
	if (Manager != nullptr)
	{
		Manager->IgnoreDepthOffset -= IgnoreIncrement;

		for (TMap<FString, FNUTStackTrace*>::TConstIterator It(Manager->Traces); It; ++It)
		{
			It.Value()->IgnoreDepthOffset -= IgnoreIncrement;
		}
	}
}


/**
 * FLogHookManager
 */

FLogHookManager::~FLogHookManager()
{
	DisableLogHook();
}

FLogHookID FLogHookManager::AddLogHook(FLogHook&& InHookFunc, FString InMatchStr, ELogHookType InHookType/*=ELogHookType::Partial*/)
{
	FLogHookID HookID = NextHookID++;

	if (LogHooks.Num() == 0)
	{
		EnableLogHook();
	}

	LogHooks.Add(FLogHookEntry{MoveTemp(InHookFunc), InMatchStr, InHookType, HookID});

	return HookID;
}

void FLogHookManager::RemoveLogHook(FLogHookID InHookID)
{
	for (int32 i=0; i<LogHooks.Num(); i++)
	{
		if (LogHooks[i].HookID == InHookID)
		{
			LogHooks.RemoveAt(InHookID);
		}
	}

	if (LogHooks.Num() == 0)
	{
		DisableLogHook();
	}
}

void FLogHookManager::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (!bWithinLogSerialize)
	{
		TGuardValue<bool> SerializeGuard(bWithinLogSerialize, true);

		for (const FLogHookEntry& CurEntry : LogHooks)
		{
			if (CurEntry.HookType == ELogHookType::Full)
			{
				if (CurEntry.LogMatch == Data)
				{
					CurEntry.HookFunc(Data, Verbosity, Category);
				}
			}
			else //if (CurEntry.HookType == ELogHookType::Partial)
			{
				if (FCString::Stristr(Data, *CurEntry.LogMatch) != nullptr)
				{
					CurEntry.HookFunc(Data, Verbosity, Category);
				}
			}
		}
	}
}

void FLogHookManager::EnableLogHook()
{
	if (!GLog->IsRedirectingTo(this))
	{
		GLog->AddOutputDevice(this);
	}
}

void FLogHookManager::DisableLogHook()
{
	if (GLog != nullptr)
	{
		GLog->RemoveOutputDevice(this);
	}
}


/**
 * FLogTraceManager
 */

FLogTraceManager::~FLogTraceManager()
{
	ClearHooks();
}

void FLogTraceManager::AddLogTrace(FString LogLine, ELogTraceFlags TraceFlags)
{
	using namespace UE::NUT;

	const bool bPartial = EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Partial);

	if (!bPartial && !EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Full))
	{
		UE_LOGF(LogUnitTest, Warning, "AddLogTrace: Should specify either ELogTraceFlags::Full or ELogTraceFlags::Partial. Enabling 'Full'.");

		TraceFlags |= ELogTraceFlags::Full;
	}
	else if (bPartial && EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Full))
	{
		UE_LOGF(LogUnitTest, Warning, "AddLogTrace: Specified both ELogTraceFlags::Full and ELogTraceFlags::Partial. Defaulting to 'Partial'.");

		TraceFlags &= ~ELogTraceFlags::Full;
	}

	const bool bDumpTrace = EnumHasAnyFlags(TraceFlags, ELogTraceFlags::DumpTrace);
	const bool bDebugTrace = EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Debug);

	UE_LOGF(LogUnitTest, Log, "Adding %lslog trace for line: %ls (DumpTrace: %i, Debug: %i)",
			(bPartial ? TEXT("partial ") : TEXT("")), ToCStr(LogLine), (int32)bDumpTrace, (int32)bDebugTrace);


	if (GLogHookManager != nullptr)
	{
		FLogHook HookFunc = [LogLine, TraceFlags](const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
			{
				FScopedIncrementTraceIgnoreDepth ScopedTraceIgnore(GTraceManager, 2);

				if (EnumHasAnyFlags(TraceFlags, ELogTraceFlags::DumpTrace))
				{
					GTraceManager->TraceAndDump(LogLine);
				}
				else
				{
					GTraceManager->AddTrace(LogLine);
				}

				if (EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Debug))
				{
					// Careful - you're about to crash unless you hit 'Continue' before you hit 'Stop Debugging'
					UE_DEBUG_BREAK();
				}
			};

		FLogHookID HookID = GLogHookManager->AddLogHook(MoveTemp(HookFunc), LogLine, (bPartial ? ELogHookType::Partial : ELogHookType::Full));

		HookIDs.Add(HookID);

		if (!HookMap.Contains(LogLine))
		{
			HookMap.Add(LogLine, HookID);
		}
	}
	else
	{
		UE_LOGF(LogUnitTest, Warning, "Error adding log trace, GLogHookManager is not set.");
	}
}

void FLogTraceManager::ClearLogTrace(FString LogLine, bool bDump/*=true*/)
{
	UE_LOGF(LogUnitTest, Log, "Clearing log trace for line: %ls", ToCStr(LogLine));

	FLogHookID HookID = INDEX_NONE;

	if (HookMap.RemoveAndCopyValue(LogLine, HookID))
	{
		HookIDs.Remove(HookID);
		GLogHookManager->RemoveLogHook(HookID);

		if (bDump)
		{
			GTraceManager->Dump(LogLine, false, false);
		}
		else
		{
			GTraceManager->Clear(LogLine);
		}
	}
}

void FLogTraceManager::ClearAll(bool bDump/*=false*/)
{
	UE_LOGF(LogUnitTest, Log, "Clearing all log traces.");

	if (bDump)
	{
		for (TMap<FString, FLogHookID>::TConstIterator It(HookMap); It; ++It)
		{
			const FString& CurLogLine = It.Key();

			if (GTraceManager->ContainsTrace(CurLogLine))
			{
				GTraceManager->Dump(CurLogLine, false, false);
			}
			else
			{
				UE_LOGF(LogUnitTest, Log, "No stack traces for log trace: %ls", ToCStr(CurLogLine));
			}
		}
	}

	ClearHooks();
}

void FLogTraceManager::ClearHooks()
{
	if (GLogHookManager != nullptr)
	{
		for (FLogHookID HookID : HookIDs)
		{
			GLogHookManager->RemoveLogHook(HookID);
		}
	}

	HookIDs.Empty();
	HookMap.Empty();
}


/**
 * FLogCommandManager
 */

FLogCommandManager::~FLogCommandManager()
{
	ClearHooks();
}

void FLogCommandManager::AddLogCommand(FString LogLine, FString Command)
{
	if (GLogHookManager != nullptr)
	{
		UE_LOGF(LogUnitTest, Log, "Adding log command '%ls' for line: %ls", ToCStr(Command), ToCStr(LogLine));

		FLogHook HookFunc = [Command](const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
			{
				GEngine->Exec(NUTUtil::GetPrimaryWorld(), *Command);
			};

		FLogCommandEntry NewEntry;

		NewEntry.LogLine = LogLine;
		NewEntry.Command = Command;
		NewEntry.HookID = GLogHookManager->AddLogHook(MoveTemp(HookFunc), LogLine);

		LogCommands.Add(MoveTemp(NewEntry));
	}
	else
	{
		UE_LOGF(LogUnitTest, Warning, "Error adding log command, GLogHookManager is not set.");
	}
}

void FLogCommandManager::RemoveByLog(FString LogLine)
{
	for (int32 i=0; i<LogCommands.Num(); i++)
	{
		const FLogCommandEntry& CurEntry = LogCommands[i];

		if (CurEntry.LogLine == LogLine)
		{
			if (GLogHookManager != nullptr)
			{
				GLogHookManager->RemoveLogHook(CurEntry.HookID);
			}

			LogCommands.RemoveAt(i);

			break;
		}
	}
}

void FLogCommandManager::RemoveByCommand(FString Command)
{
	for (int32 i=0; i<LogCommands.Num(); i++)
	{
		const FLogCommandEntry& CurEntry = LogCommands[i];

		if (CurEntry.Command == Command)
		{
			if (GLogHookManager != nullptr)
			{
				GLogHookManager->RemoveLogHook(CurEntry.HookID);
			}

			LogCommands.RemoveAt(i);

			break;
		}
	}
}

void FLogCommandManager::ClearHooks()
{
	if (GLogHookManager != nullptr)
	{
		for (const FLogCommandEntry& CurEntry : LogCommands)
		{
			GLogHookManager->RemoveLogHook(CurEntry.HookID);
		}
	}

	LogCommands.Empty();
}

}

/**
 * FLogStackTraceManager
 */

#if 1
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FLogStackTraceManager::~FLogStackTraceManager()
{
	if (GLog != nullptr)
	{
		GLog->RemoveOutputDevice(this);
	}
}

void FLogStackTraceManager::AddLogTrace(FString LogLine, ELogTraceFlags TraceFlags)
{
	const bool bPartial = EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Partial);

	if (!bPartial && !EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Full))
	{
		UE_LOGF(LogUnitTest, Warning, "AddLogTrace: Should specify either ELogTraceFlags::Full or ELogTraceFlags::Partial. Enabling 'Full'.");

		TraceFlags |= ELogTraceFlags::Full;
	}
	else if (bPartial && EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Full))
	{
		UE_LOGF(LogUnitTest, Warning, "AddLogTrace: Specified both ELogTraceFlags::Full and ELogTraceFlags::Partial. Defaulting to 'Partial'.");

		TraceFlags &= ~ELogTraceFlags::Full;
	}

	const bool bDumpTrace = EnumHasAnyFlags(TraceFlags, ELogTraceFlags::DumpTrace);
	const bool bDebugTrace = EnumHasAnyFlags(TraceFlags, ELogTraceFlags::Debug);

	UE_LOGF(LogUnitTest, Log, "Adding %lslog trace for line: %ls (DumpTrace: %i, Debug: %i)",
			(bPartial ? TEXT("partial ") : TEXT("")), *LogLine, (int32)bDumpTrace, (int32)bDebugTrace);

	// Add the log hook, if not active already
	if (!GLog->IsRedirectingTo(this))
	{
		GLog->AddOutputDevice(this);
	}

	FLogTraceEntry CurEntry;

	CurEntry.LogLine = LogLine;
	CurEntry.TraceFlags = TraceFlags;


	auto MatchLogLine =
		[&](const FLogTraceEntry& InEntry)
		{
			return InEntry.LogLine == LogLine;
		};

	if (bPartial)
	{
		if (!PartialMatches.ContainsByPredicate(MatchLogLine))
		{
			PartialMatches.Add(CurEntry);
		}
	}
	else
	{
		if (!ExactMatches.ContainsByPredicate(MatchLogLine))
		{
			ExactMatches.Add(CurEntry);
		}
	}
}

void FLogStackTraceManager::ClearLogTrace(FString LogLine, bool bDump/*=true*/)
{
	UE_LOGF(LogUnitTest, Log, "Clearing log trace for line: %ls", *LogLine);

	auto MatchLogLine =
		[&](const FLogTraceEntry& CurEntry)
		{
			return CurEntry.LogLine == LogLine;
		};

	int32 ExactIdx = ExactMatches.IndexOfByPredicate(MatchLogLine);
	int32 PartialIdx = PartialMatches.IndexOfByPredicate(MatchLogLine);

	if (ExactIdx != INDEX_NONE)
	{
		ExactMatches.RemoveAt(ExactIdx);
	}

	if (PartialIdx != INDEX_NONE)
	{
		PartialMatches.RemoveAt(PartialIdx);
	}


	if (PartialIdx != INDEX_NONE || ExactIdx != INDEX_NONE)
	{
		if (bDump)
		{
			GTraceManager->Dump(LogLine, false, false);
		}
		else
		{
			GTraceManager->Clear(LogLine);
		}
	}

	if (PartialMatches.Num() == 0 && ExactMatches.Num() == 0)
	{
		GLog->RemoveOutputDevice(this);
	}
}

void FLogStackTraceManager::ClearAll(bool bDump/*=false*/)
{
	UE_LOGF(LogUnitTest, Log, "Clearing all log traces.");

	if (bDump)
	{
		for (const FLogTraceEntry& CurEntry : ExactMatches)
		{
			if (GTraceManager->ContainsTrace(CurEntry.LogLine))
			{
				GTraceManager->Dump(CurEntry.LogLine, false, false);
			}
			else
			{
				UE_LOGF(LogUnitTest, Log, "No stack traces for log trace: %ls", *CurEntry.LogLine);
			}
		}

		for (auto CurEntry : PartialMatches)
		{
			if (GTraceManager->ContainsTrace(CurEntry.LogLine))
			{
				GTraceManager->Dump(CurEntry.LogLine, false, false);
			}
			else
			{
				UE_LOGF(LogUnitTest, Log, "No stack traces for (partial) log trace: %ls", *CurEntry.LogLine);
			}
		}
	}

	ExactMatches.Empty();
	PartialMatches.Empty();

	GLog->RemoveOutputDevice(this);
}

void FLogStackTraceManager::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	static bool bWithinLogTrace = false;

	if (!bWithinLogTrace)
	{
		auto CheckCurEntry = [](const FLogTraceEntry& CurEntry)
		{
			bWithinLogTrace = true;

			if (EnumHasAnyFlags(CurEntry.TraceFlags, ELogTraceFlags::DumpTrace))
			{
				GTraceManager->TraceAndDump(CurEntry.LogLine);
			}
			else
			{
				GTraceManager->AddTrace(CurEntry.LogLine);
			}

			if (EnumHasAnyFlags(CurEntry.TraceFlags, ELogTraceFlags::Debug))
			{
				// Careful - you're about to crash unless you hit 'Continue' before you hit 'Stop Debugging'
				UE_DEBUG_BREAK();
			}

			bWithinLogTrace = false;
		};

		for (const FLogTraceEntry& CurEntry : ExactMatches)
		{
			if (CurEntry.LogLine == Data)
			{
				CheckCurEntry(CurEntry);
			}
		}

		for (const FLogTraceEntry& CurEntry : PartialMatches)
		{
			if (FCString::Stristr(Data, *CurEntry.LogLine) != nullptr)
			{
				CheckCurEntry(CurEntry);
			}
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif


/**
 * NUTDebug
 */

FString NUTDebug::HexDump(const TArray<uint8>& InBytes, bool bDumpASCII/*=true*/, bool bDumpOffset/*=true*/)
{
	FString ReturnValue;

	if (bDumpOffset)
	{
		// Spacer for row offsets
		ReturnValue += TEXT("Offset  ");

		// Spacer between offsets and hex
		ReturnValue += TEXT("  ");

		// Top line offsets
		ReturnValue += TEXT("00 01 02 03  04 05 06 07  08 09 0A 0B  0C 0D 0E 0F");

		ReturnValue += LINE_TERMINATOR LINE_TERMINATOR;
	}

	for (int32 ByteRow=0; ByteRow<((InBytes.Num()-1) / 16)+1; ByteRow++)
	{
		FString HexRowStr;
		FString ASCIIRowStr;

		for (int32 ByteColumn=0; ByteColumn<16; ByteColumn++)
		{
			int32 ByteElement = (ByteRow*16) + ByteColumn;
			uint8 CurByte = (ByteElement < InBytes.Num() ? InBytes[ByteElement] : 0);

			if (ByteElement < InBytes.Num())
			{
				HexRowStr += FString::Printf(TEXT("%02X"), CurByte);

				// Printable ASCII-range limit
				if (bDumpASCII)
				{
					if (CurByte >= 0x20 && CurByte <= 0x7E)
					{
						TCHAR CurChar[2];

						CurChar[0] = CurByte;
						CurChar[1] = 0;

						ASCIIRowStr += CurChar;
					}
					else
					{
						ASCIIRowStr += TEXT(".");
					}
				}
			}
			else
			{
				HexRowStr += TEXT("  ");

				if (bDumpASCII)
				{
					ASCIIRowStr += TEXT(" ");
				}
			}


			// Add padding
			if (ByteColumn < 15)
			{
				if (ByteColumn > 0 && ((ByteColumn + 1) % 4) == 0)
				{
					HexRowStr += TEXT("  ");
				}
				else
				{
					HexRowStr += TEXT(" ");
				}
			}
		}


		// Add left-hand offset
		if (bDumpOffset)
		{
			ReturnValue += FString::Printf(TEXT("%08X"), ByteRow*16);

			// Spacer between offsets and hex
			ReturnValue += TEXT("  ");
		}

		// Add hex row
		ReturnValue += HexRowStr;

		// Add ASCII row
		if (bDumpASCII)
		{
			// Spacer between hex and ASCII
			ReturnValue += TEXT("  ");

			ReturnValue += ASCIIRowStr;
		}

		ReturnValue += LINE_TERMINATOR;
	}

	return ReturnValue;
}

FString NUTDebug::BitDump(const TArray<uint8>& InBytes, bool bDumpOffset/*=true*/, bool bLSBFirst/*=false*/)
{
	FString ReturnValue;

	if (bDumpOffset)
	{
		// Spacer for row offsets
		ReturnValue += TEXT("Offset  ");

		// Spacer between offsets and bits
		ReturnValue += TEXT("  ");

		// Top line offsets
		ReturnValue += TEXT("      00       01       02       03        04       05       06       07");

		ReturnValue += LINE_TERMINATOR LINE_TERMINATOR;
	}

	for (int32 ByteRow=0; ByteRow<((InBytes.Num()-1) / 8)+1; ByteRow++)
	{
		FString BitRowStr;

		for (int32 ByteColumn=0; ByteColumn<8; ByteColumn++)
		{
			int32 ByteElement = (ByteRow*8) + ByteColumn;
			uint8 CurByte = (ByteElement < InBytes.Num() ? InBytes[ByteElement] : 0);

			if (ByteElement < InBytes.Num())
			{
				if (bLSBFirst)
				{
					BitRowStr += FString::Printf(TEXT("%u%u%u%u%u%u%u%u"), !!(CurByte & 0x01), !!(CurByte & 0x02), !!(CurByte & 0x04),
													!!(CurByte & 0x08), !!(CurByte & 0x10), !!(CurByte & 0x20), !!(CurByte & 0x40),
													!!(CurByte & 0x80));
				}
				else
				{
					BitRowStr += FString::Printf(TEXT("%u%u%u%u%u%u%u%u"), !!(CurByte & 0x80), !!(CurByte & 0x40), !!(CurByte & 0x20),
													!!(CurByte & 0x10), !!(CurByte & 0x08), !!(CurByte & 0x04), !!(CurByte & 0x02),
													!!(CurByte & 0x01));
				}
			}
			else
			{
				BitRowStr += TEXT("  ");
			}


			// Add padding
			if (ByteColumn < 7)
			{
				if (ByteColumn > 0 && ((ByteColumn + 1) % 4) == 0)
				{
					BitRowStr += TEXT("  ");
				}
				else
				{
					BitRowStr += TEXT(" ");
				}
			}
		}


		// Add left-hand offset
		if (bDumpOffset)
		{
			ReturnValue += FString::Printf(TEXT("%08X"), ByteRow*8);

			// Spacer between offsets and bits
			ReturnValue += TEXT("  ");
		}

		// Add bits row
		ReturnValue += BitRowStr + LINE_TERMINATOR;
	}

	return ReturnValue;
}


