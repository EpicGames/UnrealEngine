// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTUtilProfiler.h"

#if STATS
#include "NetcodeUnitTest.h"
#include "Stats/StatsData.h"



/**
 * FFrameProfiler
 */

void FFrameProfiler::Start()
{
	if (!bActive)
	{
		bActive = true;
		StatsPrimaryEnableAdd();

		const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();

		OnNewFrameDelegateHandle = Stats.NewFrameDelegate.AddRaw(this, &FFrameProfiler::OnNewFrame);
	}
}

void FFrameProfiler::Stop()
{
	if (bActive)
	{
		const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();

		Stats.NewFrameDelegate.Remove(OnNewFrameDelegateHandle);

		StatsPrimaryEnableSubtract();
		bActive = false;


		// @todo #JohnB: For the moment, these objects are not tracked after creation, and are responsible for deleting themselves
		delete this;
	}
}

void FFrameProfiler::OnNewFrame(int64 Frame)
{
	const FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	TArray<FStatMessage> const& StatHistory = Stats.GetCondensedHistory(Frame);

	TArray<FStatMessage> TargetStats = StatHistory.FilterByPredicate(
		[&](const FStatMessage& CurElement)
		{
			return CurElement.NameAndInfo.GetShortName() == TargetEvent;
		});


	int64 TotalFrameTime = 0;

	if (TargetStats.Num() > 0)
	{
		TotalFrameTime = Stats.GetFastThreadFrameTime(Frame, EThreadType::Game);
	}

	// If there is no 'TotalFrameTime' value, calculations and detection are impossible, so skip
	if (TotalFrameTime > 0)
	{
		// Accumulate the frame time of each stat entry of this type
		int64 TotalDuration = 0;

		for (int i=0; i<TargetStats.Num(); i++)
		{
			FStatMessage& CurMsg = TargetStats[i];

			// This class should not be used with non-cycle stats
			check(CurMsg.NameAndInfo.GetFlag(EStatMetaFlags::IsCycle));

			TotalDuration += CurMsg.GetValue_Duration();
		}

		// Since division is done using integers, the value must be pre-multiplied, or the division result is fractional/not-an-integer
		float FramePercent = (float)((TotalDuration * 10000) / TotalFrameTime) * 0.01f;

#if 0
		UE_LOGF(LogUnitTest, Log, "Targeted stat found: Duration: %u, FrameTime: %f",
				TotalDuration,
				FramePercent);

		UE_LOGF(LogUnitTest, Log, "TotalFrametime: %llu, Duration: %u", TotalFrameTime, TotalDuration);
#endif


		if ((FramePercent - (float)FramePercentThreshold) > 0.f)
		{
			// This log message is used for detection in unit test code; if you modify this, unit tests must be modified too
			UE_LOGF(LogUnitTest, Log, "Detected event '%ls' breaching FramePercentThreshold (%u).", *TargetEvent.ToString(),
					FramePercentThreshold);

			Stop();
		}

		// No code past this point, in case of self-destruct
	}
}
#endif
