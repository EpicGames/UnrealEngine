// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTUtilReflection.h"
#include "Containers/ArrayBuilder.h"
#include "Misc/TVariantMeta.h"
#include "UObject/EnumProperty.h"
#include "Templates/ValueOrError.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "NetcodeUnitTest.h"


// @todo #JohnB: With the DebugDump function, NextActionError often seems to be garbled, but can't see an obvious reason why.
//					Investigate/fix this, may be a sign of a bigger problem.

// @todo #JohnB: Review the FEnumProperty changes that were added elsewhere, and test them
//					(UPDATE: The parts I've tested, I've removed this define from - there's a fair bit left to test though)
#define UENUM_REFL 1

/**
 * Wrapper macro for cast operator returns
 *
 * @return	The value the operator should return
 */
#define CAST_RETURN(ReturnVal) \
	NotifyCastReturn(); \
	return ReturnVal;


/**
 * FVMReflection
 */

FVMReflection::FVMReflection()
	: BaseAddress(nullptr)
	, FieldAddress(nullptr)
	, bVerifiedFieldType(false)
	, bSkipFieldVerification(false)
	, bSetArrayElement(false)
	, bNextActionMustBeCast(false)
	, bIsError(false)
	, NextActionError()
	, bOutError(nullptr)
	, History()
	, OutHistoryPtr(nullptr)
	, WarnLevel(EVMRefWarning::Warn)
{
}

FVMReflection::FVMReflection(UObject* InBaseObject, EVMRefWarning InWarnLevel/*=EVMRefWarning::Warn*/)
	: FVMReflection()
{
	WarnLevel = InWarnLevel;

	if (InBaseObject != nullptr)
	{
		BaseAddress = InBaseObject;
		FieldInstance = InBaseObject->GetClass();
	}
	else
	{
		BaseAddress = nullptr;
		FieldInstance = FFieldVariant();

		SetError(TEXT("Bad InBaseObject in constructor"));
	}
}

FVMReflection::FVMReflection(FStructOnScope& InStruct, EVMRefWarning InWarnLevel/*=EVMRefWarning::Warn*/)
	: FVMReflection()
{
	WarnLevel = InWarnLevel;

	const UStruct* TargetStruct = (InStruct.IsValid() ? InStruct.GetStruct() : nullptr);

	if (TargetStruct != nullptr)
	{
		BaseAddress = InStruct.GetStructMemory();
		FieldInstance = const_cast<UStruct*>(TargetStruct);
	}
	else
	{
		BaseAddress = nullptr;
		FieldInstance = FFieldVariant();

		SetError(TEXT("Bad TargetStruct in constructor"));
	}
}

FVMReflection::FVMReflection(const FVMReflection& ToCopy)
	: BaseAddress(ToCopy.BaseAddress)
	, FieldInstance(ToCopy.FieldInstance)
	, FieldAddress(ToCopy.FieldAddress)
	, bVerifiedFieldType(ToCopy.bVerifiedFieldType)
	, bSkipFieldVerification(ToCopy.bSkipFieldVerification)
	, bSetArrayElement(ToCopy.bSetArrayElement)
	, bNextActionMustBeCast(ToCopy.bNextActionMustBeCast)
	, bIsError(ToCopy.bIsError)
	, NextActionError(ToCopy.NextActionError)
	, bOutError(NULL)
	, History()
	, OutHistoryPtr(nullptr)
	, WarnLevel(EVMRefWarning::Warn)
{
}

FVMReflection::FVMReflection(FFuncReflection& InFuncRefl, EVMRefWarning InWarnLevel/*=EVMRefWarning::Warn*/)
	: FVMReflection(InFuncRefl.ParmsRefl)
{
	WarnLevel = InWarnLevel;
}

FVMReflection& FVMReflection::operator = (const FVMReflection& ToCopy)
{
	UNIT_ASSERT(FString(TEXT("This should never be called.")) == TEXT(""));
	return *this;
}

FVMReflection& FVMReflection::operator ->*(FString PropertyName)
{
	FString CurOperation = FString::Printf(TEXT("->*\"%s\""), *PropertyName);

	NotifyOperator(CurOperation);
	AddHistory(CurOperation);

	if (!bIsError && FieldInstance.IsValid())
	{
		/**
		 * Property Context
		 *
		 * Before evaluating an actual property, first parse it from within the current reflection helper context;
		 * the context is most often the current object the reflection helper is pointed at
		 */

		// UClass
		if (FieldInstance.IsA(UClass::StaticClass()))
		{
			const UClass* ClassInstance = Cast<UClass>(FieldInstance.ToUObject());
			FProperty* FoundProperty = FindFProperty<FProperty>(ClassInstance, *PropertyName);

			if (FoundProperty != nullptr)
			{
				FieldInstance = FoundProperty;
				SetFieldAddress(FoundProperty->ContainerPtrToValuePtr<void>(BaseAddress));
			}
			else
			{
				FString Error = FString::Printf(TEXT("Property '%s' not found in class '%s'"), *PropertyName,
										(ClassInstance != nullptr ? *ClassInstance->GetFullName() : TEXT("nullptr")));

				SetError(Error);
			}
		}
		// UStruct
		else if (FieldInstance.IsA(UStruct::StaticClass()))
		{
			if (!IsPropertyArray() || (bVerifiedFieldType && bSetArrayElement))
			{
				const UStruct* InnerStruct = Cast<UStruct>(FieldInstance.ToUObject());
				FProperty* FoundProperty = (InnerStruct != nullptr ? FindFProperty<FProperty>(InnerStruct, *PropertyName) : nullptr);

				if (FoundProperty != nullptr)
				{
					FieldInstance = FoundProperty;
					SetFieldAddress(FoundProperty->ContainerPtrToValuePtr<void>(BaseAddress));
				}
				else
				{
					FString Error = FString::Printf(TEXT("Property '%s' not found in struct '%s'"), *PropertyName,
											(InnerStruct != nullptr ? *InnerStruct->GetFullName() : TEXT("nullptr")));

					SetError(Error);
				}
			}
			else if (!bVerifiedFieldType)
			{
				SetError(TEXT("Can't access struct array without verifying array type."));
			}
			else // if (!bSetArrayElement)
			{
				SetError(TEXT("Can't access struct array without selecting element."));
			}
		}


		/**
		 * Property
		 *
		 * Now that the context of the property has been setup, handle some special cases which change BaseAddress,
		 * i.e. change the object being pointed to
		 */

		// These types are supported without any extra effort from within this function
		// (though some, such as arrays, need extra support elsewhere)
		static const TArray<FFieldClass*> SupportedTypes = TArrayBuilder<FFieldClass*>()
														//.Add(UClass::StaticClass())
														.Add(FByteProperty::StaticClass())
														.Add(FEnumProperty::StaticClass())
														.Add(FUInt16Property::StaticClass())
														.Add(FUInt32Property::StaticClass())
														.Add(FUInt64Property::StaticClass())
														.Add(FInt8Property::StaticClass())
														.Add(FInt16Property::StaticClass())
														.Add(FIntProperty::StaticClass())
														.Add(FInt64Property::StaticClass())
														.Add(FFloatProperty::StaticClass())
														.Add(FDoubleProperty::StaticClass())
														.Add(FBoolProperty::StaticClass())
														.Add(FNameProperty::StaticClass())
														.Add(FStrProperty::StaticClass())
														.Add(FTextProperty::StaticClass())
														.Add(FArrayProperty::StaticClass());

		auto IsSupportedType =
			[&](const FFieldClass* InClass)
			{
				return SupportedTypes.ContainsByPredicate(
					[&](FFieldClass* CurEntry)
					{
						return InClass->IsChildOf(CurEntry);
					});
			};


		// FObjectProperty
		// This is a special circumstance, context changes to the object, but FieldAddress is kept pointing to the object property
		if (IsPropertyObject())
		{
			ProcessObjectProperty();
		}
		// FStructProperty
		// Same principle as FObjectProperty, but changing context to a UStruct
		else if (FieldInstance.IsA(FStructProperty::StaticClass()))
		{
			ProcessStructProperty();
		}
		else
		{
			FString UnsupportedFieldClassName;
			if (FieldInstance.ToUObject() && !FieldInstance.ToUObject()->IsA(UClass::StaticClass()))
			{
				UnsupportedFieldClassName = FieldInstance.ToUObject()->GetClass()->GetFullName();
			}
			else if (FieldInstance.ToField() && !IsSupportedType(FieldInstance.ToField()->GetClass()))
			{
				UnsupportedFieldClassName = FieldInstance.ToField()->GetClass()->GetName();
			}
			if (!UnsupportedFieldClassName.IsEmpty())
			{
				FString Error = FString::Printf(TEXT("Support for field type '%s' has not been implemented."),
					*UnsupportedFieldClassName);

				SetError(Error);
			}
		}
	}
	else if (!bIsError && !FieldInstance.IsValid())
	{
		SetError(TEXT("FieldInstance is nullptr"));
	}

	return *this;
}

FVMReflection& FVMReflection::operator [](int32 ArrayElement)
{
	FString CurOperation = FString::Printf(TEXT("[%i]"), ArrayElement);

	NotifyOperator(CurOperation);
	AddHistory(CurOperation);

	const FProperty* FieldProp = CastField<FProperty>(FieldInstance.ToField());
	const FArrayProperty* ArrayProp = CastField<FArrayProperty>(FieldInstance.ToField());

	if (!bIsError && FieldProp != nullptr && FieldAddress != nullptr && !bSetArrayElement)
	{
		// Static arrays
		if (FieldProp->ArrayDim > 1)
		{
			if (ArrayElement >= 0 && ArrayElement < FieldProp->ArrayDim)
			{
				SetFieldAddress(FieldProp->ContainerPtrToValuePtr<void>(BaseAddress, ArrayElement), true);
			}
			else
			{
				SetError(FString::Printf(TEXT("Tried to access array element '%i' of '%i'."), ArrayElement, FieldProp->ArrayDim));
			}
		}
		// Dynamic arrays
		else if (ArrayProp != nullptr)
		{
			FScriptArrayHelper DynArray(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(BaseAddress));

			if (ArrayElement >= 0 && ArrayElement < DynArray.Num())
			{
				// Update the field type, to the arrays inner property (i.e. uint8 for a uint8 array)
				FieldInstance = ArrayProp->Inner;

				SetFieldAddress(DynArray.GetRawPtr(ArrayElement), true);
			}
			else
			{
				SetError(FString::Printf(TEXT("Tried to access array element '%i' of '%i'."), ArrayElement, FieldProp->ArrayDim));
			}
		}
		else
		{
			SetError(FString::Printf(TEXT("Property '%s' is not an array."), *FieldProp->GetName()));
		}


		// Handle context changes from FObjectProperty
		// This is a special circumstance, context changes to the object, but FieldAddress is kept pointing to the object property
		if (IsPropertyObject())
		{
			ProcessObjectProperty();
		}
		// Handle context changes from FStructProperty (same principle as above)
		else if (FieldInstance.IsA(FStructProperty::StaticClass()))
		{
			ProcessStructProperty();
		}
	}
	else if (!bIsError)
	{
		if (!FieldInstance.IsValid())
		{
			SetError(TEXT("FieldInstance is nullptr."));
		}
		else if (FieldProp == nullptr)
		{
			SetError(FString::Printf(TEXT("Field '%s' is not a property."), *FieldInstance.GetName()));
		}
		else if (FieldAddress == nullptr)
		{
			SetError(TEXT("FieldAddress is nullptr (should already be pointing to base property address)."));
		}
		else // if (bSetArrayElement)
		{
			SetError(TEXT("Array element was already set."));
		}
	}

	return *this;
}

FVMReflection& FVMReflection::operator [](const ANSICHAR* InFieldType)
{
	FString ExpectedFieldType = ANSI_TO_TCHAR(InFieldType);
	FString CurOperation = FString::Printf(TEXT("[\"%s\"]"), *ExpectedFieldType);

	NotifyOperator(CurOperation);
	AddHistory(CurOperation);

	FProperty* FieldProp = CastField<FProperty>(FieldInstance.ToField());
	FArrayProperty* ArrayProp = CastField<FArrayProperty>(FieldProp);
	UStruct* StructField = Cast<UStruct>(FieldInstance.ToUObject());

	if (!bIsError && FieldInstance.IsValid() && FieldAddress != nullptr && !bVerifiedFieldType)
	{
		FFieldVariant ActualFieldType;
		FString ActualFieldTypeStr = TEXT("");
		const TCHAR* CheckType = TEXT("");

		// Static arrays
		if (FieldProp != nullptr && FieldProp->ArrayDim > 1)
		{
			ActualFieldType = FieldProp;
			CheckType = TEXT("array");
		}
		// Dynamic arrays
		else if (ArrayProp != nullptr)
		{
			ActualFieldType = ArrayProp->Inner;
			CheckType = TEXT("array");
		}
		// Structs
		else if (StructField != nullptr)
		{
			ActualFieldType = StructField;
			CheckType = TEXT("struct");
		}
		else
		{
			SetError(FString::Printf(TEXT("Property '%s' is not a %s."), *FieldInstance.GetName(), CheckType));
		}

		// Handle special case of struct-arrays
		const FStructProperty* StructProp = CastField<FStructProperty>(ActualFieldType.ToField());

		if (StructProp != nullptr)
		{
			ActualFieldType = StructProp->Struct;
			CheckType = TEXT("struct array");
		}


		if (ActualFieldType.IsValid())
		{
			bool bTypeValid = false;

			ActualFieldTypeStr = ActualFieldType.GetClassName();

			// @todo #JohnB: Try to do away with these macros, not pretty.
			#define FIELD_TYPE_CHECK_FIRST(TypeString, TypeProperty) \
				if (ExpectedFieldType == TypeString) \
				{ \
					if (ActualFieldType.IsA(TypeProperty::StaticClass())) \
					{ \
						bTypeValid = true; \
					} \
				}

			#define FIELD_TYPE_CHECK(TypeString, TypeProperty)	else FIELD_TYPE_CHECK_FIRST(TypeString, TypeProperty)

			FIELD_TYPE_CHECK_FIRST(TEXT("bool"), FBoolProperty)
			FIELD_TYPE_CHECK(TEXT("FName"), FNameProperty)
			FIELD_TYPE_CHECK(TEXT("uint8"), FByteProperty)
			FIELD_TYPE_CHECK(TEXT("double"), FDoubleProperty)
			FIELD_TYPE_CHECK(TEXT("float"), FFloatProperty)
			FIELD_TYPE_CHECK(TEXT("int16"), FInt16Property)
			FIELD_TYPE_CHECK(TEXT("int64"), FInt64Property)
			FIELD_TYPE_CHECK(TEXT("int8"), FInt8Property)
			FIELD_TYPE_CHECK(TEXT("int32"), FIntProperty)
			FIELD_TYPE_CHECK(TEXT("uint16"), FUInt16Property)
			FIELD_TYPE_CHECK(TEXT("uint32"), FUInt32Property)
			FIELD_TYPE_CHECK(TEXT("uint64"), FUInt64Property)
			FIELD_TYPE_CHECK(TEXT("FString"), FStrProperty)
			FIELD_TYPE_CHECK(TEXT("FText"), FTextProperty)
#if UENUM_REFL
			else if (ActualFieldType.IsA(FEnumProperty::StaticClass()))
			{
				FProperty* UnderlyingActualFieldType = CastField<FEnumProperty>(ActualFieldType.ToField())->GetUnderlyingProperty();
				if (ExpectedFieldType == "uint8" && UnderlyingActualFieldType->IsA<FByteProperty>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "int16" && UnderlyingActualFieldType->IsA<FInt16Property>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "int64" && UnderlyingActualFieldType->IsA<FInt64Property>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "int8" && UnderlyingActualFieldType->IsA<FInt8Property>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "int32" && UnderlyingActualFieldType->IsA<FIntProperty>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "uint16" && UnderlyingActualFieldType->IsA<FUInt16Property>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "uint32" && UnderlyingActualFieldType->IsA<FUInt32Property>())
				{
					bTypeValid = true;
				}
				else if (ExpectedFieldType == "uint64" && UnderlyingActualFieldType->IsA<FUInt64Property>())
				{
					bTypeValid = true;
				}
				else
				{
					SetError(FString::Printf(TEXT("Tried to verify %s as being of type '%s', but it has underlying type '%s' instead."),
								CheckType, *ExpectedFieldType, *UnderlyingActualFieldType->GetClass()->GetName()));
				}
			}
#endif
			// UObject and subclasses
			else if (ExpectedFieldType.Len() > 2 &&
						(ExpectedFieldType.Left(1) == TEXT("U") || ExpectedFieldType.Left(1) == TEXT("A")) &&
						ExpectedFieldType.Right(1) == TEXT("*"))
			{
				FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>(ActualFieldType.ToField());

				if (ObjProp != nullptr)
				{
					FString ClassName = ExpectedFieldType.Mid(1, ExpectedFieldType.Len()-2);

					ActualFieldTypeStr = ObjProp->PropertyClass->GetName();

					if (ActualFieldTypeStr == ClassName)
					{
						bTypeValid = true;
					}
					else
					{
						SetError(FString::Printf(TEXT("Expected object %s of type '%s', but got %s of type '%s%s*'"),
									CheckType, *ExpectedFieldType, CheckType, *ActualFieldTypeStr, *ClassName));
					}
				}
			}
			// UStruct
			else if (ExpectedFieldType.Len() > 1 && ExpectedFieldType.Left(1) == TEXT("F"))
			{
				const UStruct* StructRef = Cast<UStruct>(ActualFieldType.ToUObject());

				if (StructRef != nullptr)
				{
					FString ClassName = ExpectedFieldType.Mid(1);

					ActualFieldTypeStr = StructRef->GetName();

					if (ActualFieldTypeStr == ClassName)
					{
						bTypeValid = true;
					}
					else
					{
						SetError(FString::Printf(TEXT("Expected %s of type '%s', but got %s of type 'F%s'"),
									CheckType, *ExpectedFieldType, CheckType, *ActualFieldTypeStr));
					}
				}
			}

			#undef FIELD_TYPE_CHECK
			#undef FIELD_TYPE_CHECK_FIRST


			if (bTypeValid)
			{
				bVerifiedFieldType = true;
			}
			else
			{
				SetError(FString::Printf(TEXT("Tried to verify %s as being of type '%s', but it is of type '%s' instead."),
							CheckType, *ExpectedFieldType, *ActualFieldTypeStr));
			}
		}
		else
		{
			SetError(FString::Printf(TEXT("Could not determine inner %s type."), CheckType));
		}
	}
	else if (!bIsError)
	{
		if (!FieldInstance.IsValid())
		{
			SetError(TEXT("FieldInstance is nullptr."));
		}
		else if (FieldAddress == nullptr)
		{
			SetError(TEXT("FieldAddress is nullptr (should already be pointing to base property address)."));
		}
		else //if (bVerifiedFieldType)
		{
			SetError(TEXT("Field type already verified."));
		}
	}

	return *this;
}

void FVMReflection::ProcessObjectProperty()
{
	if (!IsPropertyArray() || (bVerifiedFieldType && bSetArrayElement))
	{
		if (FieldInstance.IsA(FObjectProperty::StaticClass()))
		{
			UObject* PropValue = *(UObject**)FieldAddress;

			if (PropValue != nullptr)
			{
				BaseAddress = PropValue;
				FieldInstance = PropValue->GetClass();
			}
			else
			{
				bNextActionMustBeCast = true;
				NextActionError = FString::Printf(TEXT("FObjectProperty '%s' was nullptr."), *FieldInstance.GetFullName());
			}
		}
		else if (FieldInstance.IsA(FWeakObjectProperty::StaticClass()))
		{
			FWeakObjectPtr& PtrValue = *(FWeakObjectPtr*)FieldAddress;

			if (PtrValue.IsValid())
			{
				UObject* PropValue = PtrValue.Get();

				BaseAddress = PropValue;
				FieldInstance = PropValue->GetClass();
			}
			else
			{
				bNextActionMustBeCast = true;
				NextActionError = FString::Printf(TEXT("FWeakObjectProperty '%s' was Invalid."), *FieldInstance.GetFullName());
			}
		}
		else if (FieldInstance.IsA(FSoftObjectProperty::StaticClass()))
		{
			FSoftObjectPtr& PtrValue = *(FSoftObjectPtr*)FieldAddress;

			if (PtrValue.IsValid())
			{
				UObject* PropValue = PtrValue.Get();

				BaseAddress = PropValue;
				FieldInstance = PropValue->GetClass();
			}
			else
			{
				bNextActionMustBeCast = true;
				NextActionError = FString::Printf(TEXT("FSoftObjectProperty '%s' was Invalid."), *FieldInstance.GetFullName());
			}
		}
		else
		{
			FString Error = FString::Printf(
							TEXT("ProcessObjectProperty called with field '%s' of type '%s', instead of type 'FObjectProperty'"),
							*FieldInstance.GetName(), *FieldInstance.GetClassName());

			SetError(Error);
		}
	}
}

void FVMReflection::ProcessStructProperty()
{
	if (!IsPropertyArray() || (bVerifiedFieldType && bSetArrayElement))
	{
		FStructProperty* StructProp = FieldInstance.Get<FStructProperty>();

		if (StructProp != nullptr && FieldAddress != nullptr)
		{
			BaseAddress = FieldAddress;
			FieldInstance = StructProp->Struct;
		}
		else if (FieldAddress == nullptr)
		{
			SetError(TEXT("ProcessStructProperty called with FieldAddress == nullptr"));
		}
		else
		{
			bNextActionMustBeCast = true;

			NextActionError = FString::Printf(
							TEXT("ProcessStructProperty called with field '%s' of type '%s', instead of type 'FStructProperty'"),
							*FieldInstance.GetName(), *FieldInstance.GetClassName());
		}
	}
}


template<typename InType, class InTypeClass>
InType* FVMReflection::GetWritableCast(const TCHAR* InTypeStr, bool bDoingUpCast/*=false*/)
{
	InType* ReturnVal = nullptr;

	AddCastHistory(FString::Printf(TEXT("(%s*)"), InTypeStr));

	if (CanCastProperty())
	{
		if (FieldInstance.IsA(InTypeClass::StaticClass()))
		{
			ReturnVal = (InType*)FieldAddress;
		}
#if UENUM_REFL
		else if (FieldInstance.IsA<FEnumProperty>() && FieldInstance.Get<FEnumProperty>()->GetUnderlyingProperty()->IsA(InTypeClass::StaticClass()))
		{
			ReturnVal = (InType*)FieldAddress;
		}
#endif
		else if (!bDoingUpCast)
		{
			FString Error = FString::Printf(TEXT("Tried to cast type '%s' to type '%s'."), *FieldInstance.GetClassName(),
						*InTypeClass::StaticClass()->GetName());

			SetCastError(Error);
		}
	}
	else
	{
		if (IsPropertyArray() && (!bVerifiedFieldType || !bSetArrayElement))
		{
			if (!bVerifiedFieldType)
			{
				SetCastError(TEXT("Can't cast array property, verification type not set."));
			}
			else // if (!bSetArrayElement)
			{
				SetCastError(TEXT("Can't cast array property, element not set."));
			}
		}
		else
		{
			SetCastError(TEXT("Can't cast property."));
		}
	}

	CAST_RETURN(ReturnVal);
}

template<typename InType, class InTypeClass>
InType FVMReflection::GetNumericTypeCast(const TCHAR* InTypeStr, const TArray<FFieldClass*>& SupportedUpCasts)
{
	InType ReturnVal = 0;
	InType* ValuePtr = GetWritableCast<InType, InTypeClass>(InTypeStr, (SupportedUpCasts.Num() > 0));

	AddCastHistory(FString::Printf(TEXT("(%s)"), InTypeStr));

	if (ValuePtr != nullptr)
	{
		ReturnVal = *ValuePtr;
	}
	// NOTE: Can't check SupportedUpCasts against FieldInstance->GetClass(), as FieldInstance may be a child of a support cast
	else if (CanCastProperty())
	{
		#define NUMERIC_UPCAST_FIRST(CastType, CastTypeClass) \
			if (FieldInstance.IsA(CastTypeClass::StaticClass())) \
			{ \
				if (SupportedUpCasts.Contains(CastTypeClass::StaticClass())) \
				{ \
					ReturnVal = (InType)(GetTypeCast<CastType>(InTypeStr)); \
				} \
				else \
				{ \
					FString Error = FString::Printf(TEXT("Type '%s' does not support upcasting to type '%s'."), \
										*FieldInstance.GetClassName(), *InTypeClass::StaticClass()->GetName()); \
					SetCastError(Error); \
				} \
			}

		#define NUMERIC_UPCAST(CastType, CastTypeClass) else NUMERIC_UPCAST_FIRST(CastType, CastTypeClass)

		{
			NUMERIC_UPCAST_FIRST(uint8, FByteProperty)
			NUMERIC_UPCAST(uint16, FUInt16Property)
			NUMERIC_UPCAST(uint32, FUInt32Property)
			NUMERIC_UPCAST(int8, FInt8Property)
			NUMERIC_UPCAST(int16, FInt16Property)
			NUMERIC_UPCAST(int32, FIntProperty)
			NUMERIC_UPCAST(float, FFloatProperty)
#if UENUM_REFL
			else if (FieldInstance.IsA(FEnumProperty::StaticClass()))
			{
				FEnumProperty* EnumFieldInstance = FieldInstance.Get<FEnumProperty>();
				FNumericProperty* UnderlyingProp = EnumFieldInstance->GetUnderlyingProperty();
				const FFieldClass* UnderlyingPropClass = UnderlyingProp->GetClass();

				if (SupportedUpCasts.Contains(UnderlyingPropClass))
				{
					if (UnderlyingPropClass == FByteProperty::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<uint8>(InTypeStr));
					}
					else if (UnderlyingPropClass == FUInt16Property::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<uint16>(InTypeStr));
					}
					else if (UnderlyingPropClass == FUInt32Property::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<uint32>(InTypeStr));
					}
					else if (UnderlyingPropClass == FUInt64Property::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<uint64>(InTypeStr));
					}
					else if (UnderlyingPropClass == FInt8Property::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<int8>(InTypeStr));
					}
					else if (UnderlyingPropClass == FInt16Property::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<int16>(InTypeStr));
					}
					else if (UnderlyingPropClass == FIntProperty::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<int32>(InTypeStr));
					}
					else if (UnderlyingPropClass == FInt64Property::StaticClass())
					{
						ReturnVal = (InType)(GetTypeCast<int64>(InTypeStr));
					}
					else
					{
						FString Error = FString::Printf(
											TEXT("Enum property with underlying type '%s' does not support upcasting to type '%s'."),
											*UnderlyingPropClass->GetName(), *InTypeClass::StaticClass()->GetName());

						SetCastError(Error);
					}
				}
				else
				{
					FString Error = FString::Printf(TEXT("Type '%s' does not support upcasting to type '%s'."),
										*FieldInstance.GetClassName(), *InTypeClass::StaticClass()->GetName());
					SetCastError(Error);
				}
			}
#endif
			else
			{
				FString Error = FString::Printf(TEXT("No upcast possible from type '%s' to type '%s'."),
									*FieldInstance.GetClassName(), *InTypeClass::StaticClass()->GetName());

				SetCastError(Error);
			}
		}


		#undef NUMERIC_UPCAST
		#undef NUMERIC_UPCAST_FIRST
	}
	else
	{
		if (IsPropertyArray() && (!bVerifiedFieldType || !bSetArrayElement))
		{
			if (!bVerifiedFieldType)
			{
				SetCastError(TEXT("Can't cast array property, verification type not set."));
			}
			else // if (!bSetArrayElement)
			{
				SetCastError(TEXT("Can't cast array property, element not set."));
			}
		}
		else
		{
			SetCastError(TEXT("Can't cast property."));
		}
	}

	CAST_RETURN(ReturnVal);
}

template<typename InType>
FORCEINLINE InType FVMReflection::GetTypeCast(const TCHAR* InTypeStr)
{
	InType ReturnVal = 0;
	InType* ValuePtr = (InType*)(*this);

	AddCastHistory(FString::Printf(TEXT("(%s)"), InTypeStr));

	if (ValuePtr != NULL)
	{
		ReturnVal = *ValuePtr;
	}
	else
	{
		SetCastError(TEXT("Failed to get writable cast result."));
	}

	CAST_RETURN(ReturnVal);
}


/**
 * Implements a pointer cast operator (can't figure out how to do this with just the template)
 *
 * @param InType		The type being cast to
 * @param InTypeClass	The FProperty class being cast to
 */
#define IMPLEMENT_GENERIC_POINTER_CAST(InType, InTypeClass) \
	FVMReflection::operator InType*() \
	{ \
		return GetWritableCast<InType, InTypeClass>(TEXT(#InType)); \
	}

#define IMPLEMENT_NUMERIC_POINTER_CAST(InType, InTypeClass) IMPLEMENT_GENERIC_POINTER_CAST(InType, InTypeClass)

/**
 * Implements a readonly cast operator, for a numeric type
 *
 * @param InType		The type being cast to
 * @param InTypeClass	The FProperty class being cast to (not used here, but keep for clarity/consistency)
 */
#define IMPLEMENT_NUMERIC_CAST_BASIC(InType, InTypeClass) \
	FVMReflection::operator InType() \
	{ \
		return GetTypeCast<InType>(TEXT(#InType)); \
	}

/**
 * Implements a readonly cast operator, for a numeric type, with support for additional upcast types
 *
 * @param InType				The type being cast to
 * @param InTypeClass			The FProperty class being cast to
 * @param InSupportedUpCasts	A TArrayBuilder listing additional supported cast types
 */
#define IMPLEMENT_NUMERIC_CAST(InType, InTypeClass, InSupportedUpCasts) \
	FVMReflection::operator InType() \
	{ \
		static const TArray<FFieldClass*> InType##InTypeClass##UpCasts = InSupportedUpCasts; \
		return GetNumericTypeCast<InType, InTypeClass>(TEXT(#InType), InType##InTypeClass##UpCasts); \
	}

// Implement numeric pointer casts
IMPLEMENT_NUMERIC_POINTER_CAST(uint8, FByteProperty);
IMPLEMENT_NUMERIC_POINTER_CAST(uint16, FUInt16Property);
IMPLEMENT_NUMERIC_POINTER_CAST(uint32, FUInt32Property);
IMPLEMENT_NUMERIC_POINTER_CAST(uint64, FUInt64Property);
IMPLEMENT_NUMERIC_POINTER_CAST(int8, FInt8Property);
IMPLEMENT_NUMERIC_POINTER_CAST(int16, FInt16Property);
IMPLEMENT_NUMERIC_POINTER_CAST(int32, FIntProperty);
IMPLEMENT_NUMERIC_POINTER_CAST(int64, FInt64Property);
IMPLEMENT_NUMERIC_POINTER_CAST(float, FFloatProperty);
IMPLEMENT_NUMERIC_POINTER_CAST(double, FDoubleProperty);


// Implement readonly numeric casts
IMPLEMENT_NUMERIC_CAST_BASIC(uint8, FByteProperty);

IMPLEMENT_NUMERIC_CAST(uint16, FUInt16Property, TArrayBuilder<FFieldClass*>()
													.Add(FByteProperty::StaticClass()));

IMPLEMENT_NUMERIC_CAST(uint32, FUInt32Property, TArrayBuilder<FFieldClass*>()
													.Add(FByteProperty::StaticClass())
													.Add(FUInt16Property::StaticClass()));

IMPLEMENT_NUMERIC_CAST(uint64, FUInt64Property, TArrayBuilder<FFieldClass*>()
													.Add(FByteProperty::StaticClass())
													.Add(FUInt16Property::StaticClass())
													.Add(FUInt32Property::StaticClass()));

IMPLEMENT_NUMERIC_CAST_BASIC(int8, FInt8Property);

IMPLEMENT_NUMERIC_CAST(int16, FInt16Property, TArrayBuilder<FFieldClass*>()
													.Add(FInt8Property::StaticClass()));

IMPLEMENT_NUMERIC_CAST(int32, FIntProperty, TArrayBuilder<FFieldClass*>()
													.Add(FInt8Property::StaticClass())
													.Add(FInt16Property::StaticClass()));

IMPLEMENT_NUMERIC_CAST(int64, FInt64Property, TArrayBuilder<FFieldClass*>()
													.Add(FInt8Property::StaticClass())
													.Add(FInt16Property::StaticClass())
													.Add(FIntProperty::StaticClass()));

IMPLEMENT_NUMERIC_CAST_BASIC(float, FFloatProperty);

IMPLEMENT_NUMERIC_CAST(double, FDoubleProperty, TArrayBuilder<FFieldClass*>()
													.Add(FFloatProperty::StaticClass()));


// Implement generic pointer casts
IMPLEMENT_GENERIC_POINTER_CAST(FName, FNameProperty);
IMPLEMENT_GENERIC_POINTER_CAST(FString, FStrProperty);
IMPLEMENT_GENERIC_POINTER_CAST(FText, FTextProperty);


FVMReflection::operator bool()
{
	bool bReturnVal = false;
	FBoolProperty* BoolProp = FieldInstance.Get<FBoolProperty>();

	AddCastHistory(TEXT("(bool)"));

	if (CanCastProperty())
	{
		if (BoolProp != nullptr)
		{
			bReturnVal = BoolProp->GetPropertyValue(FieldAddress);
		}
		else
		{
			FString Error = FString::Printf(TEXT("FieldInstance is of type '%s', not 'FBoolProperty'."),
								*FieldInstance.GetClassName());

			SetCastError(Error);
		}
	}
	else
	{
		SetCastError(TEXT("Can't cast property."));
	}

	CAST_RETURN(bReturnVal);
}

FVMReflection::operator FName()
{
	FName ReturnVal = NAME_None;
	FName* ValuePtr = (FName*)(*this);

	AddCastHistory(TEXT("(FName)"));

	if (ValuePtr != NULL)
	{
		ReturnVal = *ValuePtr;
	}
	else
	{
		SetCastError(TEXT("Failed to get writable cast result."));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator FString()
{
	FString ReturnVal = TEXT("");

	if (const FEnumProperty* EnumProp = FieldInstance.Get<FEnumProperty>())
	{
		UEnum* TargetEnum = EnumProp->GetEnum();
		uint8 EnumValue = (uint8)(*this);

		AddCastHistory(TEXT("(FString)"));

		ReturnVal = TargetEnum->GetNameStringByValue(EnumValue);
	}
	else if (const FByteProperty* ByteProp = FieldInstance.Get<FByteProperty>())
	{
		UEnum* TargetEnum = ByteProp->Enum;
		uint8 EnumValue = (uint8)(*this);

		AddCastHistory(TEXT("(FString)"));

		ReturnVal = TargetEnum->GetNameStringByValue(EnumValue);
	}
	else
	{
		FString* ValuePtr = (FString*)(*this);

		AddCastHistory(TEXT("(FString)"));

		if (ValuePtr != nullptr)
		{
			ReturnVal = *ValuePtr;
		}
		else
		{
			SetCastError(TEXT("Failed to get writable cast result."));
		}
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator FText()
{
	FText ReturnVal = FText::GetEmpty();
	FText* ValuePtr = (FText*)(*this);

	AddCastHistory(TEXT("(FText)"));

	if (ValuePtr != NULL)
	{
		ReturnVal = *ValuePtr;
	}
	else
	{
		SetCastError(TEXT("Failed to get writable cast result."));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator UObject**()
{
	UObject** ReturnVal = nullptr;

	AddCastHistory(TEXT("(UObject**)"));

	if (CanCastObject<FObjectProperty>())
	{
		if (FieldAddress != nullptr)
		{
			ReturnVal = (UObject**)FieldAddress;
		}
		else
		{
			SetCastError(TEXT("FieldAddress is NULL"));
		}
	}
	else
	{
		SetCastError(TEXT("Can't cast object."));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator UObject*()
{
	UObject* ReturnVal = nullptr;

	AddCastHistory(TEXT("(UObject*)"));

	if (CanCastObject<FObjectProperty>())
	{
		ReturnVal = (UObject*)BaseAddress;
	}
	else
	{
		SetCastError(TEXT("Can't cast object"));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator FSoftObjectPtr*()
{
	FSoftObjectPtr* ReturnVal = nullptr;

	AddCastHistory(TEXT("(FSoftObjectPtr*)"));

	if (CanCastObject<FSoftObjectProperty>())
	{
		ReturnVal = (FSoftObjectPtr*)FieldAddress;
	}
	else
	{
		SetCastError(TEXT("Can't cast FSoftObjectPtr"));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator FScriptArray*()
{
	FScriptArray* ReturnVal = NULL;

	AddCastHistory(TEXT("(FScriptArray*)"));

	if (CanCastArray())
	{
		if (FieldAddress != NULL)
		{
			ReturnVal = (FScriptArray*)FieldAddress;
		}
		else
		{
			SetCastError(TEXT("FieldAddress is NULL"));
		}
	}
	else if (!bVerifiedFieldType)
	{
		SetCastError(TEXT("Can't cast to array, without specifying an array type for verification."));
	}
	else if (bSetArrayElement)
	{
		SetCastError(TEXT("Can't cast to array, after selecting an array element"));
	}
	else
	{
		SetCastError(TEXT("Can't cast array."));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator TSharedPtr<FScriptArrayHelper>()
{
	TSharedPtr<FScriptArrayHelper> ReturnVal = nullptr;
	FScriptArray* ScriptArray = (FScriptArray*)(*this);

	AddCastHistory(TEXT("(TSharedPtr<FScriptArrayHelper>)"));

	if (ScriptArray != NULL)
	{
		ReturnVal = MakeShareable(new FScriptArrayHelper(FieldInstance.Get<FArrayProperty>(), ScriptArray));
	}
	else
	{
		SetCastError(TEXT("Failed to get script array result."));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection::operator void*()
{
	void* ReturnVal = NULL;

	if (CanCastStruct())
	{
		if (FieldAddress != NULL)
		{
			ReturnVal = FieldAddress;
		}
		else
		{
			SetError(TEXT("FieldAddress is NULL"));
		}
	}
	else if (!bVerifiedFieldType)
	{
		SetCastError(TEXT("Can't cast to struct, without specifying a struct type for verification."));
	}
	else
	{
		SetCastError(TEXT("Can't cast struct."));
	}

	CAST_RETURN(ReturnVal);
}

FVMReflection& FVMReflection::operator = (bool Value)
{
	FBoolProperty* BoolProp = FieldInstance.Get<FBoolProperty>();

	AddHistory(Value ? TEXT(" = true") : TEXT(" = false"));

	if (CanCastProperty())
	{
		if (BoolProp != nullptr)
		{
			BoolProp->SetPropertyValue(FieldAddress, Value);
		}
		else
		{
			FString Error = FString::Printf(TEXT("FieldInstance is of type '%s', not 'FBoolProperty'."),
								*FieldInstance.GetClassName());

			SetError(Error);
		}
	}
	else
	{
		SetError(TEXT("Can't cast property."));
	}

	return *this;
}

FVMReflection& FVMReflection::operator = (UObject* Value)
{
	UObject** ObjRef = (UObject**)(*this);

	if (ObjRef != nullptr)
	{
		*ObjRef = Value;
	}

	return *this;
}

FVMReflection& FVMReflection::operator = (TCHAR* Value)
{
	if (FEnumProperty* EnumProp = FieldInstance.Get<FEnumProperty>())
	{
		UEnum* TargetEnum = EnumProp->GetEnum();
		int64 EnumIdx = (int64)TargetEnum->GetIndexByName(Value);

		if (EnumIdx != INDEX_NONE)
		{
			EnumProp->GetUnderlyingProperty()->SetIntPropertyValue(FieldAddress, EnumIdx);
		}
		else
		{
			SetError(FString::Printf(TEXT("Name '%s' is not a valid name within enum '%s'."), Value, *TargetEnum->GetName()));
		}
	}
	else if (FByteProperty* ByteProp = FieldInstance.Get<FByteProperty>())
	{
		UEnum* TargetEnum = ByteProp->Enum;
		int8 EnumIdx = (int8)TargetEnum->GetIndexByName(Value);

		if (EnumIdx != INDEX_NONE)
		{
			*(uint8*)FieldAddress = EnumIdx;
		}
		else
		{
			SetError(FString::Printf(TEXT("Name '%s' is not a valid name within enum '%s'."), Value, *TargetEnum->GetName()));
		}
	}
	else if (FSoftObjectProperty* SoftObjProp = FieldInstance.Get<FSoftObjectProperty>())
	{
		FSoftObjectPtr* SoftObjPtr = (FSoftObjectPtr*)(*this);

		if (SoftObjPtr != nullptr)
		{
			*SoftObjPtr = FSoftObjectPath(Value);
		}
	}
	else if (FNameProperty* NameProp = FieldInstance.Get<FNameProperty>())
	{
		FName* TargetName = (FName*)(*this);

		if (TargetName != nullptr)
		{
			*TargetName = Value;
		}
	}
	// Otherwise revert to string assign
	else
	{
		FString* VarRef = (FString*)(*this);

		if (VarRef != nullptr)
		{
			*VarRef = Value;
		}
	}

	return *this;
}

TValueOrError<FString, FString> FVMReflection::GetValueAsString()
{
	TValueOrError<FString, FString> ReturnVal = MakeError(FString(TEXT("")));

	if (CanCastObject<FObjectProperty>())
	{
		ReturnVal = MakeValue(FString(BaseAddress != nullptr ? ((UObject*)BaseAddress)->GetFullName() : TEXT("nullptr")));
	}
	// @todo #JohnB: I think this path is hit when there is a UFunction, not sure if it's possible for UClass to be encountered here
	// @todo #JohnB: This code path doesn't seem to be getting hit at all, that's strange.
	// @todo #JohnB: Revisit this, and see if this code path is getting hit
	else if (CanCastStruct())
	{
		const UScriptStruct* Struct = Cast<UScriptStruct>(FieldInstance.ToUObject());
		UObject* Obj = (UObject*)BaseAddress;

		if (Struct != nullptr && Obj != nullptr)
		{
			FString Result;

			Struct->ExportText(Result, FieldAddress, FieldAddress, Obj, PPF_None, nullptr);

			ReturnVal = MakeValue(Result);
		}
		else if (Cast<UStruct>(FieldInstance.ToUObject()) != nullptr && Obj != nullptr)
		{
			ReturnVal = MakeError(FString::Printf(TEXT("(Got UStruct type '%s' when expecting UScriptStruct, need to add support)"),
													*FieldInstance.GetClassName()));
		}
		else
		{
			ReturnVal = MakeValue(FString(TEXT("(nullptr)")));
		}
	}
	else if (CanCastProperty() || CanCastArray())
	{
		// @todo #JohnB: Static arrays

		FProperty* Prop = FieldInstance.Get<FProperty>();
		UObject* Obj = (UObject*)BaseAddress;

		if (Prop != nullptr && Obj != nullptr)
		{
			FString Result;

			Prop->ExportTextItem_Direct(Result, FieldAddress, FieldAddress, Obj, PPF_None);

			ReturnVal = MakeValue(Result);
		}
		else
		{
			if (CastField<FArrayProperty>(Prop) != nullptr)
			{
				ReturnVal = MakeValue(FString(TEXT("(nullptr)")));
			}
			else
			{
				ReturnVal = MakeValue(FString(TEXT("nullptr")));
			}
		}
	}
	else
	{
		ReturnVal = MakeError(FString(TEXT("ReflErr: Can't convert value to string")));
	}

	return ReturnVal;
}


FVMReflection& FVMReflection::operator ,(bool* bErrorPointer)
{
	bOutError = bErrorPointer;

	if (bOutError != NULL)
	{
		*bOutError = bIsError;
	}

	return *this;
}

FVMReflection& FVMReflection::operator ,(FString* OutHistory)
{
	OutHistoryPtr = OutHistory;

	if (OutHistoryPtr != NULL)
	{
		*OutHistoryPtr = TEXT("");

		for (auto CurString : History)
		{
			*OutHistoryPtr += CurString;
		}
	}

	return *this;
}


void FVMReflection::DebugDump()
{
	UE_LOGF(LogUnitTest, Log, "FVMReflection Dump:");

	UE_LOGF(LogUnitTest, Log, "     - BaseAddress: %ls", (BaseAddress != NULL ? TEXT("Valid") : TEXT("NULL")));

	UE_LOGF(LogUnitTest, Log, "     - FieldInstance: %ls",
			(FieldInstance.IsValid() ? *FieldInstance.GetFullName() : TEXT("NULL")));
	UE_LOGF(LogUnitTest, Log, "     - FieldAddress: %ls", (FieldAddress != NULL ? TEXT("Valid") : TEXT("NULL")));
	UE_LOGF(LogUnitTest, Log, "     - bVerifiedFieldType: %i", (int32)bVerifiedFieldType);
	UE_LOGF(LogUnitTest, Log, "     - bSkipFieldVerification: %i", (int32)bSkipFieldVerification);
	UE_LOGF(LogUnitTest, Log, "     - bSetArrayElement: %i", (int32)bSetArrayElement);

	UE_LOGF(LogUnitTest, Log, "     - bNextActionMustBeCast: %i", (int32)!!bNextActionMustBeCast);
	UE_LOGF(LogUnitTest, Log, "     - NextActionError: %ls", *NextActionError);

	UE_LOGF(LogUnitTest, Log, "     - bIsError: %i", (int32)!!bIsError);
	UE_LOGF(LogUnitTest, Log, "     - bOutError: %ls", (bOutError != NULL ? TEXT("Valid") : TEXT("NULL")));

	FString FullHistory = TEXT("");

	for (auto CurEntry : History)
	{
		FullHistory += CurEntry;
	}

	UE_LOGF(LogUnitTest, Log, "     - History: %ls", *FullHistory);
	UE_LOGF(LogUnitTest, Log, "     - OutHistoryPtr: %ls", (OutHistoryPtr != NULL ? TEXT("Valid") : TEXT("NULL")));

	UE_LOGF(LogUnitTest, Log, "     - WarnLevel: %i (%ls)",
			(int32)WarnLevel, (WarnLevel == EVMRefWarning::Warn ? TEXT("Warn") : TEXT("NoWarn")));


	bIsError = true;
}


void FVMReflection::SetFieldAddress(void* InFieldAddress, bool bSettingArrayElement/*=false*/)
{
	bool bWasAtArrayElement = bSetArrayElement && !bSettingArrayElement;

	FieldAddress = InFieldAddress;
	bSetArrayElement = bSettingArrayElement;

	// If we were at an array element, and are traversing past it now, make sure the array type was verified (a tiny bit hacky)
	if (bWasAtArrayElement && !bVerifiedFieldType)
	{
		SetError(TEXT("Array type was not specified for verification."));
	}

	// Whenever we set the FieldAddress for a non-array, reset array type verification status
	if (!bSettingArrayElement)
	{
		bVerifiedFieldType = bSkipFieldVerification;
	}
}


void FVMReflection::SetError(FString InError, bool bCastError/*=false*/)
{
	if (!bIsError)
	{
		bIsError = true;

		if (bOutError != NULL)
		{
			*bOutError = bIsError;
		}

		if (InError.Len() > 0)
		{
			FString HistoryStr = FString::Printf(TEXT(" (ReflErr: %s)"), *InError);

			if (bCastError)
			{
				// The cast should already be in history at this point, at position 0, so add the error to position 1
				History.Insert(HistoryStr, 1);
			}
			else
			{
				History.Add(HistoryStr);
			}
		}

		if (OutHistoryPtr != NULL)
		{
			*OutHistoryPtr = TEXT("");

			for (auto CurStr : History)
			{
				*OutHistoryPtr += CurStr;
			}
		}


		if (WarnLevel == EVMRefWarning::Warn)
		{
			FString CurHistory;

			for (auto CurString : History)
			{
				CurHistory += CurString;
			}

			UE_LOGF(LogUnitTest, Log, "Reflection Err: History dump: %ls", *CurHistory);
		}
	}
}


/**
 * NUTUtilRefl
 */

FString NUTUtilRefl::FunctionParmsToString(UFunction* InFunction, void* Parms)
{
	FString ReturnVal = TEXT("");

	for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & (CPF_Parm|CPF_ReturnParm)) == CPF_Parm; ++It)
	{
		FString CurPropText;

		// Set PortFlags to maximize property coverage
		It->ExportTextItem_Direct(CurPropText, It->ContainerPtrToValuePtr<void>(Parms), nullptr, nullptr,
							PPF_IncludeTransient | PPF_DuplicateForPIE);

		if (ReturnVal.Len() > 0)
		{
			ReturnVal += TEXT(", ");
		}

		ReturnVal += It->GetName() + TEXT(" = ") + CurPropText;
	}

	return ReturnVal;
}

