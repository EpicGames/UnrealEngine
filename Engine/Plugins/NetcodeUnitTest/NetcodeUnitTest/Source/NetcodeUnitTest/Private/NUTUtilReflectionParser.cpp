// Copyright Epic Games, Inc. All Rights Reserved.

#include "NUTUtilReflectionParser.h"

#include "Misc/AutomationTest.h"
#include "Engine/Engine.h"
#include "Misc/ExpressionParser.h"
#include "UObject/UObjectIterator.h"

#include "Misc/TVariantMeta.h"
#include "NUTUtilReflection.h"


// @todo #JohnB: Add special keywords or operators, for getting the list of functions/variables in an object too.

// @todo #JohnB: Implement function calls properly at some stage. Right now they only work as special keyword functions,
//					at the start of the string

// @todo #JohnB: Implement the assign '=' operator, for enabling set-command type usage.

// @todo #JohnB: The default 'Find' keyword excludes CDO's/archetypes - add FindDefault/FindCDO,
//					FindArc/FindArchetype and FindAny functions


/**
 * Structs for defining operators/tokens that the parser should implement
 */

/**
 * Token for marking the dot operator, i.e. A.B
 */
struct FReflDotOp
{
};

/**
 * Token for marking the array subscript operator, i.e. Array[Num]
 */
struct FReflArraySubOp
{
	int32 ElementIndex;

	FReflArraySubOp(int32 InElementIndex)
		: ElementIndex(InElementIndex)
	{
	}
};


/**
 * Token for marking identifiers, i.e. variable/function names
 */
struct FIdentifier
{
	FString IdentifierName;

	FIdentifier(FString InIdentifierName)
		: IdentifierName(InIdentifierName)
	{
	}
};

/**
 * Token for marking function call start bracket: (
 */
struct FFuncStart
{
};

/**
 * Token for marking function call end bracket: )
 */
struct FFuncEnd
{
};

/**
 * Token for marking the function parameter separator: ,
 */
struct FFuncParamSeparator
{
};

DEFINE_EXPRESSION_NODE_TYPE(FReflDotOp, 0xE55EF40E, 0x821E8B60, 0x2BA256B9, 0x521499DB);
DEFINE_EXPRESSION_NODE_TYPE(FReflArraySubOp, 0x0499497C, 0xBE6AA0FC, 0x17EFF7DC, 0x30E53C8C);
DEFINE_EXPRESSION_NODE_TYPE(FIdentifier, 0xFFA64890, 0x8371A963, 0xCCAE15B7, 0xBFB9F3B8);
DEFINE_EXPRESSION_NODE_TYPE(FFuncStart, 0x8D928207, 0xC34DFEDB, 0xAEFC11D0, 0x7CD0592F);
DEFINE_EXPRESSION_NODE_TYPE(FFuncEnd, 0x83FB289C, 0xA3008A1B, 0xE3BB4332, 0x997D612A);
DEFINE_EXPRESSION_NODE_TYPE(FFuncParamSeparator, 0x88AC556F, 0x5D250FFB, 0xC13DD565, 0x6BE3860E);


/**
 * FVMReflectionParser
 */

FVMReflectionParser::FVMReflectionParser()
{
	TokenDefinitions.IgnoreWhitespace();


	// Define a token for parsing identifiers
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			FTokenStream& Stream = Consumer.GetStream();

			TOptional<FStringToken> Token = ParseIdentifier(Stream);

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FIdentifier(Token.GetValue().GetString()));
			}

			return TOptional<FExpressionError>();
		});

	// Define a token for parsing the dot operator
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol('.');

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FReflDotOp());
			}

			return TOptional<FExpressionError>();
		});

	// @todo #JohnB: Return to this later, this one is a bit more complicated
#if 0
	// Define a token for parsing the array subscript operator
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			FTokenStream& Stream = Consumer.GetStream();

			TOptional<FStringToken> Token = ParseArraySubscript(Stream);

			if (Token.IsSet())
			{
				int32 ElementIndex = FCString::Atoi(*Token.GetValue().GetString());
				Consumer.Add(Token.GetValue(), FReflArraySubOp(ElementIndex));
			}

			return TOptional<FExpressionError>();
		});
#endif

	// Define a token for parsing the function call start bracket
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol('(');

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FFuncStart());
			}

			return TOptional<FExpressionError>();
		});

	// Define a token for parsing the function call end bracket
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol(')');

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FFuncEnd());
			}

			return TOptional<FExpressionError>();
		});

	// Define a token for parsing the function parameter separator
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol(',');

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FFuncParamSeparator());
			}

			return TOptional<FExpressionError>();
		});


	Grammar.DefineGrouping<FFuncStart, FFuncEnd>();
	Grammar.DefineBinaryOperator<FReflDotOp>(2, EAssociativity::LeftToRight);


	// Ignores parameter A, the current state is passed through Context instead
	auto ExecDotOp = [](FIdentifier B, const FContextPointer* Context) -> FExpressionResult
	{
		FReflEvaluationContext* RealContext = (Context != nullptr ? Context->Context : nullptr);

		if (RealContext != nullptr)
		{
			FVMReflection& Refl = *(RealContext->Refl);

			Refl->*(B.IdentifierName);

			if (Refl.IsError())
			{
				return MakeError(FText::FromString(FString::Printf(TEXT("Reflection error. History: %s"),
									*Refl.GetHistory())));
			}
		}
		else
		{
			return MakeError(FText::FromString(TEXT("Invalid Context.")));
		}

		// Always return a blank identifier
		return MakeValue(FIdentifier(TEXT("BlankDotOpReturn")));
	};

	OpJumpTable.MapBinary<FReflDotOp>(
		[ExecDotOp](FIdentifier A, FIdentifier B, const FContextPointer* Context) -> FExpressionResult
		{
			return ExecDotOp(B, Context);
		});

	// Dot operator coming after a function call
	OpJumpTable.MapBinary<FReflDotOp>(
		[ExecDotOp](FFuncEnd A, FIdentifier B, const FContextPointer* Context) -> FExpressionResult
		{
			return ExecDotOp(B, Context);
		});
}

TValueOrError<TSharedPtr<FVMReflection>, FExpressionError> FVMReflectionParser::Evaluate(const TCHAR* InExpression,
																							UObject* InTargetObj/*=nullptr*/) const
{
	TValueOrError<TArray<FExpressionToken>, FExpressionError> LexResult = ExpressionParser::Lex(InExpression, TokenDefinitions);

	if (!LexResult.IsValid())
	{
		return MakeError(LexResult.StealError());
	}


	TArray<FExpressionToken> Tokens = LexResult.StealValue();
	FReflEvaluationContext Context;

	if (Tokens.Num() > 0)
	{
		FStringToken TokenContext = Tokens[0].Context;
		const FExpressionNode& FirstNode = Tokens[0].Node;
		const FIdentifier* FirstIdentifier = Tokens[0].Node.Cast<FIdentifier>();

		if (FirstIdentifier != nullptr)
		{
			// Need to initialize Context, based on the first identifier being a special keyword
			if (InTargetObj == nullptr)
			{
				// @todo #JohnB: May have to make keywords available from a different part of the code,
				//					and allow for multiple reflection objects in one statement,
				//					if you do things such as allowing programmatic statements to e.g. the array operator.
				FString Keyword = FirstIdentifier->IdentifierName;

				if (Keyword == TEXT("GEngine"))
				{
					Context.Refl = MakeShareable(new FVMReflection(GEngine));
				}
				else if (Keyword == TEXT("Find") || Keyword == TEXT("FindObj") || Keyword == TEXT("FindObject"))
				{
					bool bParseSuccess = false;
					FString ObjStr = TEXT("");
					FString ClassStr;
					int32 FindTokenCount = 1;

					if (Tokens.Num() >= 4)
					{
						enum class EFuncParseStage
						{
							StartBracket,
							ObjName,
							ParamSeparator,
							ClassName,
							EndBracket
						};

						EFuncParseStage Stage = EFuncParseStage::StartBracket;

						bParseSuccess = true;

						for (int32 TokenIdx=1; TokenIdx<Tokens.Num() && bParseSuccess; TokenIdx++, FindTokenCount++)
						{
							FExpressionNode& CurNode = Tokens[TokenIdx].Node;

							if (Stage == EFuncParseStage::StartBracket)
							{
								bParseSuccess = CurNode.Cast<FFuncStart>() != nullptr;
								Stage = EFuncParseStage::ObjName;
							}
							else if (Stage == EFuncParseStage::ObjName)
							{
								const FIdentifier* ObjIdentifier = CurNode.Cast<FIdentifier>();

								// Allow a blank Object name, if you just want to search by class
								if (ObjIdentifier == nullptr && CurNode.Cast<FFuncParamSeparator>() != nullptr)
								{
									bParseSuccess = true;
									Stage = EFuncParseStage::ClassName;
								}
								else
								{
									ObjStr = (ObjIdentifier != nullptr ? ObjIdentifier->IdentifierName : TEXT(""));
									bParseSuccess = ObjIdentifier != nullptr;
									Stage = EFuncParseStage::ParamSeparator;
								}
							}
							else if (Stage == EFuncParseStage::ParamSeparator)
							{
								// Second parameter is optional, early-out if the closing bracket is found
								if (CurNode.Cast<FFuncEnd>() != nullptr)
								{
									break;
								}

								bParseSuccess = CurNode.Cast<FFuncParamSeparator>() != nullptr;
								Stage = EFuncParseStage::ClassName;
							}
							else if (Stage == EFuncParseStage::ClassName)
							{
								const FIdentifier* ClassIdentifier = CurNode.Cast<FIdentifier>();

								ClassStr = (ClassIdentifier != nullptr ? ClassIdentifier->IdentifierName : TEXT(""));
								bParseSuccess = ClassIdentifier != nullptr;
								Stage = EFuncParseStage::EndBracket;
							}
							else // if (Stage == EFuncParseStage::EndBracket)
							{
								bParseSuccess = CurNode.Cast<FFuncEnd>() != nullptr;

								break;
							}
						}
					}
					
					// @todo #JohnB: Unfortunately, due to the dot operator, this is not as effective as it could be.
					//					If you want to allow dots here, you may need to create a new/special identifier type?
					//					Going to take some thinking through - longfinger it.
					// @todo #JohnB: I think in this case, add in support for parsing strings as a grouping with "",
					//					and make sure everything within that grouping is treated as one big string/identifier,
					//					which should sort out this problem
					//					UPDATE: Doesn't being within the function brackets () not already count as this?
					//					UPDATE2: No, it likely doesn't count as that, because you may want to allow statement
					//								parsing within the brackets in future anyway, including use of the dot operator,
					//								so only makes sense to specify dot in a string
					if (bParseSuccess)
					{
						// Remove the FindObject tokens
						Tokens.RemoveAt(0, FindTokenCount);

						UClass* FindClass = ClassStr.Len() > 0 ? UClass::TryFindTypeSlow<UClass>(ClassStr) : UObject::StaticClass();
						UObject* BestMatch = nullptr;

						if (FindClass != nullptr)
						{
							const EObjectFlags ExcludeFlags = EObjectFlags::RF_ClassDefaultObject | EObjectFlags::RF_ArchetypeObject;

							for (FThreadSafeObjectIterator It(FindClass, false, ExcludeFlags); It; ++It)
							{
								FString CurName = It->GetName();

								if (CurName == ObjStr || ObjStr.Len() == 0)
								{
									BestMatch = *It;
									break;
								}
								else if (CurName.Contains(ObjStr))
								{
									if (BestMatch == nullptr || CurName.Find(ObjStr) < BestMatch->GetName().Find(ObjStr))
									{
										BestMatch = *It;
									}
								}
							}
						}

						if (BestMatch != nullptr)
						{
							Context.Refl = MakeShareable(new FVMReflection(BestMatch));
						}
						else
						{
							return MakeError(FExpressionError(FText::FromString(
												FString::Printf(TEXT("Failed to find object matching name '%s' of class '%s'."),
												*ObjStr, *ClassStr))));
						}
					}
					else
					{
						return MakeError(FExpressionError(FText::FromString(TEXT("Failed to parse Find keyword. ")
								TEXT("Syntax: Find(ObjName) or Find(ObjName, ClassName)"))));
					}
				}
				else
				{
					return MakeError(FExpressionError(FText::FromString(FString(TEXT("Unrecognized keyword: ")) + Keyword)));
				}
			}
			// Need to initialize Context using InTargetObj, and inject a blank identifier and dot operator,
			// with parameter B being the first identifier in the string.
			else
			{
				Context.Refl = MakeShareable(new FVMReflection(InTargetObj));

				Tokens.Insert(FExpressionToken(FStringToken(), FIdentifier(TEXT("DudIdentifier"))), 0);
				Tokens.Insert(FExpressionToken(FStringToken(), FReflDotOp()), 1);
			}
		}
		else
		{
			return MakeError(FExpressionError(FText::FromString(TEXT("First parameter must be an identifier (variable/etc. name)."))));
		}
	}


	TValueOrError<TArray<FCompiledToken>, FExpressionError> CompileResult = ExpressionParser::Compile(MoveTemp(Tokens), Grammar);

	if (!CompileResult.IsValid())
	{
		return MakeError(CompileResult.StealError());
	}

	if (!Context.Refl.IsValid())
	{
		return MakeError(FExpressionError(FText::FromString(TEXT("Failed to initialize reflection helper."))));
	}


	// Disable the need to verify field type names before you can access them, otherwise reflection is more cumbersome to use
	Context.Refl->DisableFieldVerification();

	FContextPointer ContextPointer(&Context);
	TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(CompileResult.GetValue(), OpJumpTable,
																							&ContextPointer);

	if (Result.IsValid())
	{
		return MakeValue(Context.Refl);
	}
	else
	{
		return MakeError(Result.GetError());
	}
}

TValueOrError<FString, FExpressionError> FVMReflectionParser::EvaluateString(const TCHAR* InExpression,
																		UObject* InTargetObj/*=nullptr*/) const
{
	TValueOrError<TSharedPtr<FVMReflection>, FExpressionError> Result = Evaluate(InExpression, InTargetObj);

	if (Result.IsValid())
	{
		FVMReflection* Refl = Result.GetValue().Get();

		if (Refl != nullptr && !Refl->IsError())
		{
			TValueOrError<FString, FString> ReflResult = Refl->GetValueAsString();

			if (ReflResult.IsValid())
			{
				return MakeValue(ReflResult.GetValue());
			}
			else
			{
				return MakeError(FExpressionError(FText::FromString(ReflResult.GetError())));
			}
		}
		else if (Refl == nullptr)
		{
			return MakeError(FExpressionError(FText::FromString(TEXT("Bad reflection pointer."))));
		}
		else // if (Refl->IsError())
		{
			return MakeError(FExpressionError(FText::FromString(FString::Printf(TEXT("Reflection error. History: %s"),
								*Refl->GetHistory()))));
		}
	}
	else
	{
		return MakeError(Result.GetError());
	}
}

TOptional<FStringToken> FVMReflectionParser::ParseIdentifier(const FTokenStream& InStream, FStringToken* Accumulate/*=nullptr*/)
{
	bool bFirstChar = true;

	return InStream.ParseToken([&](TCHAR InChar)
		{
			EParseState ReturnVal = EParseState::Cancel;

			if (FChar::IsDigit(InChar))
			{
				// First char can't be a number (C++ standard)
				if (bFirstChar)
				{
					ReturnVal = EParseState::Cancel;
				}
				else
				{
					ReturnVal = EParseState::Continue;
				}
			}
			else if (FChar::IsAlpha(InChar) || InChar == '_')
			{
				ReturnVal = EParseState::Continue;
			}
			// @todo #JohnB: Unfortunately, it is hard to differentiate identifier strings from other things,
			//					which may share some of the same characters in the future.
			//					So you are just going to have to reserve all of a-z,A-Z,_ for identifiers,
			//					and not cancel when an odd character is encountered, unless we're parsing the first character.
			else if (!bFirstChar)
			{
				ReturnVal = EParseState::StopBefore;
			}
			else
			{
				ReturnVal = EParseState::Cancel;
			}

			bFirstChar = false;

			return ReturnVal;
		}, Accumulate);
}

TOptional<FStringToken> FVMReflectionParser::ParseArraySubscript(const FTokenStream& InStream, FStringToken* Accumulate/*=nullptr*/)
{
	// @todo #JohnB: Return to this later, this one is a bit more complicated

	enum class EArrayState
	{
		AS_SubEntry,
		AS_ElementIndex,
		AS_SubEnd
	};

	EArrayState State;

	return InStream.ParseToken([&](TCHAR InChar)
		{
			if (State == EArrayState::AS_SubEntry)
			{
				if (InChar == '[')
				{
					State = EArrayState::AS_ElementIndex;

					return EParseState::Continue;
				}
				else
				{
					return EParseState::Cancel;
				}
			}
			else if (State == EArrayState::AS_ElementIndex)
			{
				// @todo #JohnB: Perhaps just delay implementation of this operator, and get the rest of the easier things working,
				//					and return to this when the meat of the feature is functional?

				// @todo #JohnB: Need to support identifier parsing here for Vars, not just raw numbers
				//					(think this will be complicated...possibly very complicated)
				//					UPDATE: Yea, I think you will need to review how operators are implemented,
				//					as your implementation is diverging from this (I think)

				// @todo #JohnB: IMPORTANT: Maybe the array subscript should be treated as a grouping instead?
			}
			else // if (State == EArrayState::AS_SubEnd)
			{
				// @todo #JohnB
			}

			// @todo #JohnB: Remove, used to get rid of compiler error temporarily
			return EParseState::Cancel;

		}, Accumulate);
}


// Associativity tests for the expression parser (implemented here rather than Core, as the parser here is better suited to the tests)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
struct FLeftAssociativityOp
{
};

struct FRightAssociativityOp
{
};

DEFINE_EXPRESSION_NODE_TYPE(FLeftAssociativityOp, 0x8B3158D6, 0xE417E1F3, 0xD9D85759, 0x93246A83);
DEFINE_EXPRESSION_NODE_TYPE(FRightAssociativityOp, 0x91D9AC07, 0xD6E48925, 0x48440E7C, 0x4C5A5F68);


struct FTestResult
{
	FString ResultStr;
};


void FVMReflectionParser::TestConstruct()
{
	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol('$');

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FLeftAssociativityOp());
			}

			return TOptional<FExpressionError>();
		});

	TokenDefinitions.DefineToken([](FExpressionTokenConsumer& Consumer)
		{
			TOptional<FStringToken> Token = Consumer.GetStream().ParseSymbol('~');

			if (Token.IsSet())
			{
				Consumer.Add(Token.GetValue(), FRightAssociativityOp());
			}

			return TOptional<FExpressionError>();
		});

	Grammar.DefineBinaryOperator<FLeftAssociativityOp>(1000, EAssociativity::LeftToRight);
	Grammar.DefineBinaryOperator<FRightAssociativityOp>(1001, EAssociativity::RightToLeft);


	TestOpJumpTable.MapBinary<FLeftAssociativityOp>(
		[](FIdentifier A, FIdentifier B, const FTestResultPointer* Context) -> FExpressionResult
		{
			FTestResult* RealContext = (Context != nullptr ? Context->Result : nullptr);

			if (RealContext != nullptr)
			{
				FString& ResultStr = RealContext->ResultStr;

				if (ResultStr.Len() == 0)
				{
					ResultStr += A.IdentifierName;
				}

				ResultStr += B.IdentifierName;
			}
			else
			{
				return MakeError(FText::FromString(TEXT("Invalid Context.")));
			}

			// Always return a blank identifier
			return MakeValue(FIdentifier(TEXT("BlankDotOpReturn")));
		});

	TestOpJumpTable.MapBinary<FRightAssociativityOp>(
		[](FIdentifier A, FIdentifier B, const FTestResultPointer* Context) -> FExpressionResult
		{
			FTestResult* RealContext = (Context != nullptr ? Context->Result : nullptr);

			if (RealContext != nullptr)
			{
				FString& ResultStr = RealContext->ResultStr;

				if (ResultStr.Len() == 0)
				{
					ResultStr += A.IdentifierName;
					ResultStr += B.IdentifierName;
				}
				else
				{
					ResultStr += A.IdentifierName;
				}
			}
			else
			{
				return MakeError(FText::FromString(TEXT("Invalid Context.")));
			}

			// Always return a blank identifier
			return MakeValue(FIdentifier(TEXT("BlankDotOpReturn")));
		});
}

TValueOrError<FString, FExpressionError> FVMReflectionParser::TestEvaluate(const TCHAR* InExpression) const
{
	FTestResult TestResult;
	FTestResultPointer TestResultPointer(&TestResult);

	TValueOrError<FExpressionNode, FExpressionError> Result = ExpressionParser::Evaluate(InExpression, TokenDefinitions, Grammar,
																							TestOpJumpTable, &TestResultPointer);

	if (Result.IsValid())
	{
		return MakeValue(TestResult.ResultStr);
	}
	else
	{
		return MakeError(Result.GetError());
	}
}

bool TestExpression(FAutomationTestBase* Test, const TCHAR* Expression, const TCHAR* Expected)
{
	FVMReflectionParser Parser;

	Parser.TestConstruct();

	TValueOrError<FString, FExpressionError> Result = Parser.TestEvaluate(Expression);

	if (!Result.IsValid())
	{
		Test->AddError(Result.GetError().Text.ToString());
		return false;
	}
	
	if (Result.GetValue() != Expected)
	{
		Test->AddError(FString::Printf(TEXT("'%s' evaluation results: %s != %s"), Expression, *Result.GetValue(), Expected));
		return false;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLTRAssociativtyExpressionsTest, "System.Core.Expression Parser.LTR Operator Associativity",
									EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter);

bool FLTRAssociativtyExpressionsTest::RunTest(const FString& Parameters)
{
	TestTrue(TEXT("Expression with LeftToRight associativity did not evaluate left to right."),
				TestExpression(this, TEXT("A $ B $ C $ D $ E"), TEXT("ABCDE")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRTLAssociativtyExpressionsTest, "System.Core.Expression Parser.RTL Operator Associativity",
									EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter);

bool FRTLAssociativtyExpressionsTest::RunTest(const FString& Parameters)
{
	TestTrue(TEXT("Expression with RightToLeft associativity did not evaluate right to left."),
				TestExpression(this, TEXT("A ~ B ~ C ~ D ~ E"), TEXT("DECBA")));

	return true;
}
#endif
