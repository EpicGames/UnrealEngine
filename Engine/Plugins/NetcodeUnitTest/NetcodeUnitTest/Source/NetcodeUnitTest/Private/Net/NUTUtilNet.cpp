// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NUTUtilNet.h"

#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "EngineUtils.h"
#include "Net/RepLayout.h"

#include "NUTUtil.h"
#include "NUTUtilReflection.h"
#include "ClientUnitTest.h"
#include "MinimalClient.h"


// Forward declarations
class FWorldTickHook;


// Enable access to FRepLayout.Cmds
IMPLEMENT_GET_PRIVATE_VAR(FRepLayout, Cmds, TArray<FRepLayoutCmd>);


IMPLEMENT_FIELD(FNetPropertyHook)


/** Active unit test worlds */
TArray<UWorld*> UnitTestWorlds;

/** Unit testing worlds, pending cleanup */
TArray<UWorld*> PendingUnitWorldCleanup;

/** Active world tick hooks */
TArray<FWorldTickHook*> ActiveTickHooks;


/**
 * FProcessNetEventHook
 */

void FProcessEventHook::AddRPCHook(UWorld* InWorld, FOnProcessNetEvent InHook)
{
	PreAddHook();
	NetEventHooks.Add(InWorld, InHook);
}

void FProcessEventHook::RemoveRPCHook(UWorld* InWorld)
{
	NetEventHooks.Remove(InWorld);
	PostRemoveHook();
}

void FProcessEventHook::AddEventHook(UWorld* InWorld, FOnProcessNetEvent InHook)
{
	PreAddHook();
	EventHooks.Add(InWorld, InHook);
}

void FProcessEventHook::RemoveEventHook(UWorld* InWorld)
{
	EventHooks.Remove(InWorld);
	PostRemoveHook();
}

FDelegateHandle FProcessEventHook::AddGlobalRPCHook(FOnProcessNetEvent InHook)
{
	PreAddHook();
	GlobalNetEventHooks.Add(InHook);

	return GlobalNetEventHooks.Last().GetHandle();
}

void FProcessEventHook::RemoveGlobalRPCHook(FDelegateHandle InHandle)
{
	int32 Idx = GlobalNetEventHooks.IndexOfByPredicate(
		[&InHandle](const FOnProcessNetEvent& CurEntry)
		{
			return CurEntry.GetHandle() == InHandle;
		});

	GlobalNetEventHooks.RemoveAt(Idx);
	PostRemoveHook();
}

FDelegateHandle FProcessEventHook::AddGlobalEventHook(FOnProcessNetEvent InHook)
{
	PreAddHook();
	GlobalEventHooks.Add(InHook);

	return GlobalEventHooks.Last().GetHandle();
}

void FProcessEventHook::RemoveGlobalEventHook(FDelegateHandle InHandle)
{
	int32 Idx = GlobalEventHooks.IndexOfByPredicate(
		[&InHandle](const FOnProcessNetEvent& CurEntry)
		{
			return CurEntry.GetHandle() == InHandle;
		});

	GlobalEventHooks.RemoveAt(Idx);
	PostRemoveHook();
}

void FProcessEventHook::PreAddHook()
{
#if !UE_BUILD_SHIPPING
	if (NetEventHooks.Num() == 0 && EventHooks.Num() == 0)
	{
		AActor::ProcessEventDelegate.BindRaw(this, &FProcessEventHook::HandleProcessEvent);
	}
#else
	check(false);
#endif
}

void FProcessEventHook::PostRemoveHook()
{
#if !UE_BUILD_SHIPPING
	if (NetEventHooks.Num() == 0 && EventHooks.Num() == 0)
	{
		AActor::ProcessEventDelegate.Unbind();
	}
#else
	check(false);
#endif
}

bool FProcessEventHook::HandleProcessEvent(AActor* Actor, UFunction* Function, void* Parameters)
{
	bool bBlockEvent = false;

	if (Actor != nullptr && Function != nullptr)
	{
		bool bNetClientRPC = !!(Function->FunctionFlags & FUNC_Net) && !!(Function->FunctionFlags & FUNC_NetClient);
		bool bValidEvent = (bNetClientRPC && NetEventHooks.Num() > 0) || EventHooks.Num() > 0;

		if (bValidEvent)
		{
			UWorld* CurWorld = Actor->GetWorld();

			if (CurWorld != nullptr)
			{
				FOnProcessNetEvent* Hook = (bNetClientRPC ? NetEventHooks.Find(CurWorld) : EventHooks.Find(CurWorld));

				if (Hook != nullptr)
				{
					Hook->Execute(Actor, Function, Parameters, bBlockEvent);
				}
			}
		}

		bool bNetServerRPC = !!(Function->FunctionFlags & FUNC_Net) && !!(Function->FunctionFlags & FUNC_NetServer);

		for (const FOnProcessNetEvent& Hook : (bNetClientRPC || bNetServerRPC ? GlobalNetEventHooks : GlobalEventHooks))
		{
			Hook.Execute(Actor, Function, Parameters, bBlockEvent);
		}
	}

	return bBlockEvent;
}


/**
 * FNetworkNotifyHook
 */

EAcceptConnection::Type FNetworkNotifyHook::NotifyAcceptingConnection()
{
	EAcceptConnection::Type ReturnVal = EAcceptConnection::Ignore;

	if (NotifyAcceptingConnectionDelegate.IsBound())
	{
		ReturnVal = NotifyAcceptingConnectionDelegate.Execute();
	}

	// Until I have a use-case for doing otherwise, make the original authoritative
	if (HookedNotify != NULL)
	{
		ReturnVal = HookedNotify->NotifyAcceptingConnection();
	}

	return ReturnVal;
}

void FNetworkNotifyHook::NotifyAcceptedConnection(UNetConnection* Connection)
{
	NotifyAcceptedConnectionDelegate.ExecuteIfBound(Connection);

	if (HookedNotify != NULL)
	{
		HookedNotify->NotifyAcceptedConnection(Connection);
	}
}

bool FNetworkNotifyHook::NotifyAcceptingChannel(UChannel* Channel)
{
	bool bReturnVal = false;

	if (NotifyAcceptingChannelDelegate.IsBound())
	{
		bReturnVal = NotifyAcceptingChannelDelegate.Execute(Channel);
	}

	// Until I have a use-case for doing otherwise, make the original authoritative
	if (HookedNotify != NULL)
	{
		bReturnVal = HookedNotify->NotifyAcceptingChannel(Channel);
	}

	return bReturnVal;
}

void FNetworkNotifyHook::NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch)
{
	bool bHandled = false;

	if (NotifyControlMessageDelegate.IsBound())
	{
		bHandled = NotifyControlMessageDelegate.Execute(Connection, MessageType, Bunch);
	}

	// Only pass on to original, if the control message was not handled
	if (!bHandled && HookedNotify != NULL)
	{
		HookedNotify->NotifyControlMessage(Connection, MessageType, Bunch);
	}
}


/**
 * FWorldTickHook
 */

void FWorldTickHook::FInternalGCObject::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Owner->AttachedWorld);
}

FString FWorldTickHook::FInternalGCObject::GetReferencerName() const
{
	return TEXT("FWorldTickHook::FInternalGCObject");
}

void FWorldTickHook::Init()
{
	if (AttachedWorld != nullptr)
	{
		TickDispatchDelegateHandle  = AttachedWorld->OnTickDispatch().AddRaw(this, &FWorldTickHook::TickDispatch);
		PostTickFlushDelegateHandle = AttachedWorld->OnPostTickFlush().AddRaw(this, &FWorldTickHook::PostTickFlush);
	}
}

void FWorldTickHook::Cleanup()
{
	if (AttachedWorld != nullptr)
	{
		AttachedWorld->OnPostTickFlush().Remove(PostTickFlushDelegateHandle);
		AttachedWorld->OnTickDispatch().Remove(TickDispatchDelegateHandle);
	}

	AttachedWorld = nullptr;
}

void FWorldTickHook::TickDispatch(float DeltaTime)
{
	GActiveLogWorld = AttachedWorld;
}

void FWorldTickHook::PostTickFlush()
{
	GActiveLogWorld = nullptr;
}


/**
 * FScopedNetObjectReplace
 */

FScopedNetObjectReplace::FScopedNetObjectReplace(UClientUnitTest* InUnitTest, UObject* InObjToReplace, UObject* InObjReplacement)
	: UnitTest(InUnitTest)
	, ObjToReplace(InObjToReplace)
{
	UNetConnection* UnitConn = (UnitTest != nullptr && UnitTest->MinClient != nullptr ? UnitTest->MinClient->GetConn() : nullptr);

	if (UnitConn != nullptr)
	{
		UUnitTestPackageMap* PackageMap = Cast<UUnitTestPackageMap>(UnitConn->PackageMap);

		if (PackageMap != nullptr)
		{
			check(!PackageMap->ReplaceObjects.Contains(ObjToReplace));

			PackageMap->ReplaceObjects.Add(ObjToReplace, InObjReplacement);
		}
		else
		{
			check(false);
		}
	}
	else
	{
		check(false);
	}
}

FScopedNetObjectReplace::~FScopedNetObjectReplace()
{
	UNetConnection* UnitConn = (UnitTest != nullptr && UnitTest->MinClient != nullptr ? UnitTest->MinClient->GetConn() : nullptr);

	if (UnitConn != nullptr)
	{
		UUnitTestPackageMap* PackageMap = Cast<UUnitTestPackageMap>(UnitConn->PackageMap);

		if (PackageMap != nullptr)
		{
			check(PackageMap->ReplaceObjects.Contains(ObjToReplace));

			PackageMap->ReplaceObjects.Remove(ObjToReplace);
		}
		else
		{
			check(false);
		}
	}
	else
	{
		check(false);
	}
}

/**
 * FScopedNetNameReplace
 */

FScopedNetNameReplace::FScopedNetNameReplace(UMinimalClient* InMinClient, const FOnSerializeName::FDelegate& InDelegate)
	: MinClient(InMinClient)
	, Handle()
{
	UNetConnection* UnitConn = (MinClient != nullptr ? MinClient->GetConn() : nullptr);
	UUnitTestPackageMap* PackageMap = (UnitConn != nullptr ? Cast<UUnitTestPackageMap>(UnitConn->PackageMap) : nullptr);

	if (PackageMap != nullptr)
	{
		Handle = PackageMap->OnSerializeName.Add(InDelegate);
	}
}

FScopedNetNameReplace::FScopedNetNameReplace(UMinimalClient* InMinClient, FName InNameToReplace, FName InNameReplacement)
	: FScopedNetNameReplace(InMinClient, FOnSerializeName::FDelegate::CreateLambda(
		[InNameToReplace, InNameReplacement](bool bPreSerialize, bool& bSerializedName, FArchive&Ar, FName& InName)
		{
			if (InName == InNameToReplace)
			{
				if (Ar.IsSaving() && bPreSerialize)
				{
					InName = InNameReplacement;
				}
				else if (Ar.IsLoading() && !bPreSerialize)
				{
					InName = InNameReplacement;
				}
			}
		}))
{
}

FScopedNetNameReplace::~FScopedNetNameReplace()
{
	if (Handle.IsValid())
	{
		UNetConnection* UnitConn = (MinClient != nullptr ? MinClient->GetConn() : nullptr);
		UUnitTestPackageMap* PackageMap = (UnitConn != nullptr ? Cast<UUnitTestPackageMap>(UnitConn->PackageMap) : nullptr);

		if (PackageMap != nullptr)
		{
			PackageMap->OnSerializeName.Remove(Handle);
		}
	}
}

/**
 * FScopedRPCParamReplace
 */

FScopedRPCParamReplace::FScopedRPCParamReplace(UMinimalClient* InMinClient, UFunction* InTargetRPC, FString InParamName,
												const FOnNetSerializeItem& InSerializeHook)
	: ParamRepCmd(nullptr)
	, OriginalParam(nullptr)
{
	TSharedPtr<FRepLayout> FuncRepLayout;

	if (UNetConnection* UnitConn = (InMinClient != nullptr ? InMinClient->GetConn() : nullptr))
	{
		if (UNetDriver* NetDriver = UnitConn->GetDriver())
		{
			FuncRepLayout = NetDriver->GetFunctionRepLayout(InTargetRPC);
		}
	}

	if (FuncRepLayout.IsValid())
	{
		TArray<FRepLayoutCmd>& Cmds = GET_PRIVATE(FRepLayout, FuncRepLayout.Get(), Cmds);

		for (FRepLayoutCmd& CurCmd : Cmds)
		{
			if (FProperty* CurProp = CurCmd.Property)
			{
				if (CurProp->GetName() == InParamName)
				{
					ParamRepCmd = &CurCmd;
					OriginalParam = CurProp;

					FNetPropertyHook* HookParam = GetMutableDefault<FNetPropertyHook>();

					HookParam->SerializeHook = InSerializeHook;

					ParamRepCmd->Property = HookParam;

					break;
				}
			}
		}
	}
}


FScopedRPCParamReplace::~FScopedRPCParamReplace()
{
	GetMutableDefault<FNetPropertyHook>()->SerializeHook.Unbind();

	if (ParamRepCmd != nullptr)
	{
		ParamRepCmd->Property = OriginalParam;
	}

	ParamRepCmd = nullptr;
	OriginalParam = nullptr;
}

/**
 * FNetPropertyHook
 */

bool FNetPropertyHook::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	bool bReturnVal = false;

	if (SerializeHook.IsBound())
	{
		bReturnVal = SerializeHook.Execute(Ar, Map, Data, MetaData);
	}

	return bReturnVal;
}


/**
 * NUTNet
 */

void NUTNet::HandleBeaconReplicate(AActor* InBeacon, UNetConnection* InConnection)
{
	// Due to the way the beacon is created in unit tests (replicated, instead of taking over a local beacon client),
	// the NetDriver and BeaconConnection values have to be hack-set, to enable sending of RPC's
	InBeacon->SetNetDriverName(InConnection->Driver->NetDriverName);

	FVMReflection(InBeacon)->*"BeaconConnection" = InConnection;
}


UWorld* NUTNet::CreateUnitTestWorld(bool bHookTick/*=true*/)
{
	UWorld* ReturnVal = NULL;

	// Unfortunately, this hack is needed, to avoid a crash when running as commandlet
	// NOTE: Sometimes, depending upon build settings, PRIVATE_GIsRunningCommandlet is a define instead of a global
#ifndef PRIVATE_GIsRunningCommandlet
	bool bIsCommandlet = PRIVATE_GIsRunningCommandlet;

	PRIVATE_GIsRunningCommandlet = false;
#endif

	ReturnVal = UWorld::CreateWorld(EWorldType::None, false);

#ifndef PRIVATE_GIsRunningCommandlet
	PRIVATE_GIsRunningCommandlet = bIsCommandlet;
#endif


	if (ReturnVal != NULL)
	{
		UnitTestWorlds.Add(ReturnVal);

		// Hook the new worlds 'tick' event, so that we can capture logging
		if (bHookTick)
		{
			FWorldTickHook* TickHook = new FWorldTickHook(ReturnVal);

			ActiveTickHooks.Add(TickHook);

			TickHook->Init();
		}


		// Hack-mark the world as having begun play (when it has not)
		ReturnVal->SetBegunPlay(true);

		// Hack-mark the world as having initialized actors (to allow RPC hooks)
		ReturnVal->bActorsInitialized = true;

		// Enable pause, using the PlayerController of the primary world (unless we're in the editor)
		// @todo #JohnB: Broken in the commandlet. No LocalPlayer
		if (!GIsEditor)
		{
			AWorldSettings* CurSettings = ReturnVal->GetWorldSettings();

			if (CurSettings != NULL)
			{
				ULocalPlayer* PrimLocPlayer = GEngine->GetFirstGamePlayer(NUTUtil::GetPrimaryWorld());
				APlayerController* PrimPC = (PrimLocPlayer != NULL ? ToRawPtr(PrimLocPlayer->PlayerController) : NULL);
				APlayerState* PrimState = (PrimPC != NULL ? ToRawPtr(PrimPC->PlayerState) : NULL);

				if (PrimState != NULL)
				{
					CurSettings->SetPauserPlayerState(PrimState);
				}
			}
		}

		// Create a blank world context, to prevent crashes
		FWorldContext& CurContext = GEngine->CreateNewWorldContext(EWorldType::None);
		CurContext.SetCurrentWorld(ReturnVal);
	}

	return ReturnVal;
}

void NUTNet::MarkUnitTestWorldForCleanup(UWorld* CleanupWorld, bool bImmediate/*=false*/)
{
	UnitTestWorlds.Remove(CleanupWorld);
	PendingUnitWorldCleanup.Add(CleanupWorld);

	if (!bImmediate)
	{
		GEngine->DeferredCommands.AddUnique(TEXT("CleanupUnitTestWorlds"));
	}
	else
	{
		CleanupUnitTestWorlds();
	}
}

void NUTNet::CleanupUnitTestWorlds()
{
	for (auto CleanupIt=PendingUnitWorldCleanup.CreateIterator(); CleanupIt; ++CleanupIt)
	{
		UWorld* CurWorld = *CleanupIt;

		// Iterate all ActorComponents in this world, and unmark them as having begun play - to prevent a crash during GC
		for (TActorIterator<AActor> ActorIt(CurWorld); ActorIt; ++ActorIt)
		{
			for (UActorComponent* CurComp : ActorIt->GetComponents())
			{
				if (CurComp->HasBegunPlay())
				{
					// Big hack - call only the parent class UActorComponent::EndPlay function, such that only bHasBegunPlay is unset
					bool bBeginDestroyed = CurComp->HasAnyFlags(RF_BeginDestroyed);

					CurComp->SetFlags(RF_BeginDestroyed);

					CurComp->UActorComponent::EndPlay(EEndPlayReason::Quit);

					if (!bBeginDestroyed)
					{
						CurComp->ClearFlags(RF_BeginDestroyed);
					}
				}
			}
		}

		// Remove the tick-hook, for this world
		int32 TickHookIdx = ActiveTickHooks.IndexOfByPredicate(
			[&CurWorld](const FWorldTickHook* CurTickHook)
			{
				return CurTickHook != NULL && CurTickHook->AttachedWorld == CurWorld;
			});

		if (TickHookIdx != INDEX_NONE)
		{
			ActiveTickHooks.RemoveAt(TickHookIdx);
		}

		GEngine->DestroyWorldContext(CurWorld);
		CurWorld->DestroyWorld(false);
	}

	PendingUnitWorldCleanup.Empty();


	// Immediately garbage collect remaining objects, to finish net driver cleanup
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
}

bool NUTNet::IsUnitTestWorld(UWorld* InWorld)
{
	return UnitTestWorlds.Contains(InWorld);
}

