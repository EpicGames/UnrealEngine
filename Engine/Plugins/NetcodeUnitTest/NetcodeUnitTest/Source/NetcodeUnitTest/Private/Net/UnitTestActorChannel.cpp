// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/UnitTestActorChannel.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerController.h"


#include "MinimalClient.h"



/**
 * Default constructor
 */
UUnitTestActorChannel::UUnitTestActorChannel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MinClient(nullptr)
{
#if !UE_BUILD_SHIPPING
	bBlockChannelFailure = true;
#endif
}

void UUnitTestActorChannel::ReceivedBunch(FInBunch& Bunch)
{
	// If Actor is nullptr, and we are processing a bunch, then we are most likely initializing the actor channel
	if (Actor == nullptr)
	{
		GIsInitializingActorChan = true;
	}

	Super::ReceivedBunch(Bunch);

	GIsInitializingActorChan = false;
}

void UUnitTestActorChannel::Tick()
{
	UNetConnection* OldReceiveUnitConn = GActiveReceiveUnitConnection;

	// If Actor is nullptr and we're about to process queued bunches, we may be about to do a (delayed) actor channel initialization
	if (Actor == nullptr && QueuedBunches.Num() > 0)
	{
		GIsInitializingActorChan = true;
		GActiveReceiveUnitConnection = Connection;
	}

	Super::Tick();

	GActiveReceiveUnitConnection = OldReceiveUnitConn;
	GIsInitializingActorChan = false;
}

void UUnitTestActorChannel::NotifyActorChannelOpen(AActor* InActor, FInBunch& InBunch)
{
	APlayerController* PC = Cast<APlayerController>(InActor);

	InActor->SetNetDriverName(Connection->Driver->NetDriverName);

	// Override 'UNetConnection::HandleClientPlayer', as called through PlayerController::OnActorChannelOpen
	if (PC != nullptr)
	{
		uint8 NetPlayerIndex = 0;

		InBunch << NetPlayerIndex;

		// Does not support child connections at the moment
		check(NetPlayerIndex == 0);

		if (NetPlayerIndex == 0 && Connection->Driver != nullptr && Connection == Connection->Driver->ServerConnection)
		{
			// Implement only essential parts of the original function, as we want to block most of it (triggers level change code)
			PC->SetRole(ROLE_AutonomousProxy);
			PC->NetConnection = Connection;

			Connection->PlayerController = PC;
			Connection->OwningActor = PC;

			if (PC->Player == nullptr)
			{
				PC->Player = NewObject<ULocalPlayer>(GEngine, GEngine->LocalPlayerClass);
			}

			check(MinClient != nullptr);

			MinClient->HandleClientPlayerDel.ExecuteIfBound(PC, Connection);
		}
	}
	else
	{
		Super::NotifyActorChannelOpen(InActor, InBunch);
	}
}
