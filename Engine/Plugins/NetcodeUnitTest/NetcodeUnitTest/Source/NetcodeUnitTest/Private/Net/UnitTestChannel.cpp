// Copyright Epic Games, Inc. All Rights Reserved.


#include "Net/UnitTestChannel.h"


#include "MinimalClient.h"


/**
 * UUNitTestChannel
 */

UUnitTestChannel::UUnitTestChannel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MinClient(nullptr)
	, bVerifyOpen(false)
{
	ChName = NAME_None;
}

void UUnitTestChannel::Init(UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags)
{
	// If the channel type is still default, assume this is a control channel (since that is the only time ChType should be default)
	if (ChName == NAME_None)
	{
		if (InChIndex != 0)
		{
			UE_LOGF(LogUnitTest, Warning, "Unit test channel type was NAME_None, for non-control channel");
		}

		ChName = NAME_Control;
	}

	Super::Init(InConnection, InChIndex, CreateFlags);
}

void UUnitTestChannel::ReceivedBunch(FInBunch& Bunch)
{
	if (MinClient != nullptr)
	{
		MinClient->ReceivedControlBunchDel.ExecuteIfBound(Bunch);
	}
}

void UUnitTestChannel::Tick()
{
	Super::Tick();

	// Copied from the control channel code
	// @todo #JohnBBug: This can spam the log sometimes, upon unit test error; fix this
	if (!OpenAcked)
	{
		int32 Count = 0;

		for (FOutBunch* Out=OutRec; Out; Out=Out->Next)
		{
			if (!Out->ReceivedAck)
			{
				Count++;
			}
		}

		if (Count > 8)
		{
			return;
		}

		for (FOutBunch* Out=OutRec; Out; Out=Out->Next)
		{
			if (!Out->ReceivedAck)
			{
				const double Wait = Connection->Driver->GetElapsedTime() - Out->Time;

				if (Wait > 1.0)
				{
					UE_LOGF(LogUnitTest, Log, "UnitTestChannel %i ack timeout); resending %i...", ChIndex, Out->ChSequence);

					Connection->SendRawBunch(*Out, 0);
				}
			}
		}
	}
}

