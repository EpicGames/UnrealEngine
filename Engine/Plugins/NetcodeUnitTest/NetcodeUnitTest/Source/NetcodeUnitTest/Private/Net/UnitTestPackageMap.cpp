// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/UnitTestPackageMap.h"


#include "MinimalClient.h"
#include "NUTUtilDebug.h"


/**
 * UUnitTestPackageMap
 */

UUnitTestPackageMap::UUnitTestPackageMap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bWithinSerializeNewActor(false)
	, bPendingArchetypeSpawn(false)
	, ReplaceObjects()
	, OnSerializeName()
{
}

bool UUnitTestPackageMap::SerializeObject(FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID* OutNetGUID/*=nullptr*/)
{
	bool bReturnVal = false;

	if (Ar.IsSaving())
	{
		UObject** ObjReplacement = ReplaceObjects.Find(Obj);

		// Don't overwrite Obj when saving...
		UObject* SaveObj = (ObjReplacement != nullptr ? *ObjReplacement : Obj);

		bReturnVal = Super::SerializeObject(Ar, InClass, SaveObj, OutNetGUID);
	}
	else
	{
		bReturnVal = Super::SerializeObject(Ar, InClass, Obj, OutNetGUID);

		if (Ar.IsLoading())
		{
			UObject** ObjReplacement = ReplaceObjects.Find(Obj);

			Obj = (ObjReplacement != nullptr ? *ObjReplacement : Obj);
		}
	}

	if (bWithinSerializeNewActor)
	{
		// This indicates that SerializeObject has failed to find an existing instance when trying to serialize an actor,
		// so it will be spawned clientside later on (after the archetype is serialized) instead.
		// These spawns count as undesired clientside code execution, so filter them through NotifyAllowNetActor.
		if (InClass == AActor::StaticClass() && Obj == nullptr)
		{
			bPendingArchetypeSpawn = true;
		}
		// This indicates that a new actor archetype has just been serialized (which may or may not be during actor channel init);
		// this is the first place we know the type of a replicated actor (in an actor channel or otherwise), but BEFORE it is spawned
		else if ((GIsInitializingActorChan || bPendingArchetypeSpawn) && InClass == UObject::StaticClass() && Obj != nullptr)
		{
			bool bBlockActor = true;

			check(MinClient != nullptr);

			MinClient->RepActorSpawnDel.ExecuteIfBound(Obj->GetClass(), GIsInitializingActorChan, bBlockActor);

			if (bBlockActor)
			{
				UE_LOGF(LogUnitTest, Log,
						"Blocking replication/spawning of actor on client (add to NotifyAllowNetActor if required).");

				UE_LOGF(LogUnitTest, Log, "     ActorChannel: %ls, Class: %ls, Archetype: %ls",
						(GIsInitializingActorChan ? TEXT("true") : TEXT("false")), *Obj->GetClass()->GetFullName(),
						*Obj->GetFullName());

				Obj = nullptr;
			}
		}
	}

	return bReturnVal;
}

bool UUnitTestPackageMap::SerializeName(FArchive& Ar, FName& InName)
{
	bool bSerialized = false;
	FName SaveName = InName;

	OnSerializeName.Broadcast(true, bSerialized, Ar, (Ar.IsSaving() ? SaveName : InName));

	if (!bSerialized)
	{
		Super::SerializeName(Ar, (Ar.IsSaving() ? SaveName : InName));
	}

	OnSerializeName.Broadcast(false, bSerialized, Ar, (Ar.IsSaving() ? SaveName : InName));

	return true;
}

bool UUnitTestPackageMap::SerializeNewActor(FArchive& Ar, class UActorChannel* Channel, class AActor*& Actor)
{
	bool bReturnVal = false;

	bWithinSerializeNewActor = true;
	bPendingArchetypeSpawn = false;

	{
		// Disable PackageMap error logs during this call, since we may be deliberately causing an error to block serialization
		FScopedLogSuppress Suppress(TEXT("LogNetPackageMap"));

		bReturnVal = Super::SerializeNewActor(Ar, Channel, Actor);
	}

	bPendingArchetypeSpawn = false;
	bWithinSerializeNewActor = false;

	// If we are initializing an actor channel, then make this returns false, to block PostNetInit from being called on the actor,
	// which in turn, blocks BeginPlay being called (this can lead to actor component initialization, which can trigger garbage
	// collection issues down the line)
	if (GIsInitializingActorChan)
	{
		bReturnVal = false;
	}

	return bReturnVal;
}

