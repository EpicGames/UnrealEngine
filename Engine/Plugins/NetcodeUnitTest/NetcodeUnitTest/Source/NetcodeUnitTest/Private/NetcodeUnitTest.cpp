// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetcodeUnitTest.h"

#include "Misc/CommandLine.h"

#include "UI/LogWidgetCommands.h"

#include "INetcodeUnitTest.h"
#include "Net/NUTUtilNet.h"
#include "NUTGlobals.h"
#include "NUTUtilDebug.h"
#include "NUTUtilReflection.h"
#include "UnitTestEnvironment.h"


/**
 * Globals
 */

bool GIsInitializingActorChan = false;


/**
 * Definitions/implementations
 */

DEFINE_LOG_CATEGORY(LogUnitTest);
DEFINE_LOG_CATEGORY(NetCodeTestNone);


// @todo #JohnB: Need to rewrite Startup/Shutdown for Hot Reload support


// @todo #JohnB: IMPORTANT: Need to review all log-hooking code, and add inherent support for hot reloading,
//					as every single log hook used within unit tests, will be a source of crashes,
//					unless it unloads while a hot reload is in progress

/**
 * Module implementation
 */
class FNetcodeUnitTest : public INetcodeUnitTest
{
private:
	static FWorldDelegates::FWorldInitializationEvent::FDelegate OnWorldCreatedDelegate;

	static FDelegateHandle OnWorldCreatedDelegateHandle;

	static FDelegateHandle DumpRPCsHandle;

public:
	/**
	 * Called upon loading of the NetcodeUnitTest library
	 */
	virtual void StartupModule() override
	{
		using namespace UE::NUT;

		FNUTModuleInterface::StartupModule();

		static bool bSetDelegate = false;

		if (!bSetDelegate && FParse::Param(FCommandLine::Get(), TEXT("NUTServer")))
		{
			OnWorldCreatedDelegate = FWorldDelegates::FWorldInitializationEvent::FDelegate::CreateStatic(
										&FNetcodeUnitTest::OnWorldCreated);

			OnWorldCreatedDelegateHandle = FWorldDelegates::OnPreWorldInitialization.Add(OnWorldCreatedDelegate);

			bSetDelegate = true;
		}

		FLogWidgetCommands::Register();
		FUnitTestEnvironment::Register();


		/**
		 * Hooks all RPC calls, and dumps the function + full parameter list, to the log.
		 * Optionally, can specify string matches for filtering RPC's, delimited with ','.
		 *
		 * Examples:
		 *	-DumpRPCs
		 *
		 *	-DumpRPCs="RPC1,RPC2"
		 */
		FString MatchesStr;

		if (FParse::Param(FCommandLine::Get(), TEXT("DumpRPCs")) || FParse::Value(FCommandLine::Get(), TEXT("DumpRPCs="), MatchesStr))
		{
			if (MatchesStr.Len() > 0)
			{
				MatchesStr.ParseIntoArray(UNUTGlobals::Get().DumpRPCMatches, TEXT(","));
			}

			DumpRPCsHandle = FProcessEventHook::Get().AddGlobalRPCHook(FOnProcessNetEvent::CreateStatic(&FNetcodeUnitTest::DumpRPC));
		}

		// Add earlier log trace opportunity (execcmds is sometimes too late)
		FString TraceStr;

		if (FParse::Value(FCommandLine::Get(), TEXT("LogTrace="), TraceStr) && TraceStr.Len() > 0)
		{
			GLogTrace->AddLogTrace(TraceStr, ELogTraceFlags::Partial | ELogTraceFlags::DumpTrace);
		}

		if (FParse::Value(FCommandLine::Get(), TEXT("LogDebug="), TraceStr) && TraceStr.Len() > 0)
		{
			GLogTrace->AddLogTrace(TraceStr, ELogTraceFlags::Partial | ELogTraceFlags::Debug);
		}

		if (FParse::Value(FCommandLine::Get(), TEXT("LogCommand="), TraceStr) && TraceStr.Len() > 0)
		{
			FString LogLine;
			FString Command;

			if (TraceStr.Split(TEXT("="), &LogLine, &Command))
			{
				GLogCommandManager->AddLogCommand(LogLine, Command);
			}
			else
			{
				UE_LOGF(LogUnitTest, Warning, "LogCommand commandline parameter requires the format: -LogCommand=\"LogLine=Command\"");
			}
		}
	}

	/**
	 * Called immediately prior to unloading of the NetcodeUnitTest library
	 */
	virtual void ShutdownModule() override
	{
		using namespace UE::NUT;

		// Eliminate active global variables
		GUnitTestManager = nullptr;

		if (GTraceManager != nullptr)
		{
			delete GTraceManager;

			GTraceManager = nullptr;
		}

		if (GLogHookManager != nullptr)
		{
			delete GLogHookManager;

			GLogHookManager = nullptr;
		}

		if (GLogTrace != nullptr)
		{
			delete GLogTrace;

			GLogTrace = nullptr;
		}

		if (GLogCommandManager != nullptr)
		{
			delete GLogCommandManager;

			GLogCommandManager = nullptr;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (GLogTraceManager != nullptr)
		{
			delete GLogTraceManager;

			GLogTraceManager = nullptr;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		

		FLogWidgetCommands::Unregister();
		FUnitTestEnvironment::Unregister();

		if (DumpRPCsHandle.IsValid())
		{
			FProcessEventHook::Get().RemoveGlobalRPCHook(DumpRPCsHandle);
			DumpRPCsHandle.Reset();
		}

		FNUTModuleInterface::ShutdownModule();
	}


	/**
	 * Delegate implementation, for adding NUTActor to ServerActors, early on in engine startup
	 */
	static void OnWorldCreated(UWorld* UnrealWorld, const UWorld::InitializationValues IVS)
	{
		// If NUTActor isn't already in RuntimeServerActors, add it now
		if (GEngine != nullptr)
		{
			bool bNUTActorPresent = GEngine->RuntimeServerActors.ContainsByPredicate(
				[](const FString& CurServerActor)
				{
					return CurServerActor == TEXT("/Script/NetcodeUnitTest.NUTActor");
				});

			if (!bNUTActorPresent)
			{
				UE_LOGF(LogUnitTest, Log, "NUTActor not present in RuntimeServerActors - adding this");

				GEngine->RuntimeServerActors.Add(TEXT("/Script/NetcodeUnitTest.NUTActor"));
			}
		}

		// Now remove it, so it's only called once
		FWorldDelegates::OnPreWorldInitialization.Remove(OnWorldCreatedDelegateHandle);
	}

	// @todo #JohnB: Presently this has a 'bug' (feature?) where it logs not just received RPC's, but sent RPC's as well.
	static void DumpRPC(AActor* Actor, UFunction* Function, void* Parameters, bool& bBlockRPC)
	{
		TArray<FString>& DumpRPCMatches = UNUTGlobals::Get().DumpRPCMatches;
		FString FuncName = Function->GetName();

		bool bContainsMatch = DumpRPCMatches.Num() == 0 || DumpRPCMatches.ContainsByPredicate(
								[&FuncName](FString& CurEntry)
								{
									return FuncName.Contains(CurEntry);
								});

		if (bContainsMatch)
		{
			FString FuncParms = NUTUtilRefl::FunctionParmsToString(Function, Parameters);

			UE_LOGF(LogUnitTest, Log, "Received RPC '%ls' for actor '%ls'", *FuncName, *Actor->GetFullName());

			if (FuncParms.Len() > 0)
			{
				UE_LOGF(LogUnitTest, Log, "     '%ls' parameters: %ls", *FuncName, *FuncParms);
			}
		}
	}
};

FWorldDelegates::FWorldInitializationEvent::FDelegate FNetcodeUnitTest::OnWorldCreatedDelegate = nullptr;

FDelegateHandle FNetcodeUnitTest::OnWorldCreatedDelegateHandle;

FDelegateHandle FNetcodeUnitTest::DumpRPCsHandle;



IMPLEMENT_MODULE(FNetcodeUnitTest, NetcodeUnitTest);


/**
 * INUTModuleInterface
 */
void FNUTModuleInterface::StartupModule()
{
	check(ModuleName != nullptr);

	// Create a dependency on NetcodeUnitTest
	FModuleManager::Get().LoadModuleChecked(TEXT("NetcodeUnitTest"));


	UNUTGlobals& NUTGlobals = UNUTGlobals::Get();

	NUTGlobals.UnitTestModules.AddUnique(ModuleName);
	NUTGlobals.UnloadedModules.Remove(ModuleName);
}

void FNUTModuleInterface::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UNUTGlobals& NUTGlobals = UNUTGlobals::Get();

		if (NUTGlobals.IsValidLowLevel())
		{
			NUTGlobals.UnloadedModules.AddUnique(ModuleName);
		}
	}
}
