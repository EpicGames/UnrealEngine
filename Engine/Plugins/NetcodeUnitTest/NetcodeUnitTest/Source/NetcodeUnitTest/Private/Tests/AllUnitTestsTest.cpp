// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Misc/AutomationTest.h"

#include "UnitTestManager.h"
#include "UnitTest.h"
#include "NUTUtil.h"


// @todo #JohnBAutomation: Some unit tests may require special automation flags, and you should automatically parse these from the unit test
//				class itself, if possible (and since this runs all unit tests, you should probably iterate all unit tests that will
//				be run, to check their requirements)


/**
 * FNUTWaitForUnitTests
 *
 * Waits for all active unit tests to complete
 */

DEFINE_LATENT_AUTOMATION_COMMAND(FNUTWaitForUnitTests);

bool FNUTWaitForUnitTests::Update()
{
	return GUnitTestManager == NULL || !GUnitTestManager->IsRunningUnitTests();
}


/**
 * FAutomationConsoleCommand
 *
 * @todo #JohnBRefactor: AutomationTestCommon.h already implements this, but is private
 */
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FAutomationConsoleCommand, FString, Command);

bool FAutomationConsoleCommand::Update()
{
	if (GEngine != NULL)
	{
		GEngine->Exec(NULL, *Command);
	}

	return true;
}


/**
 * FAllUnitTestsTest
 */

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FAllUnitTestsTest, "System.Netcode Unit Test", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter);

void FAllUnitTestsTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	// Add the primary 'all unit tests' command
	OutBeautifiedNames.Add(TEXT("All Unit Tests (fast)"));
	OutTestCommands.Add(TEXT("all"));


	// Now add an entry for each individual unit test
	TArray<UUnitTest*> UnitTestClassDefaults;

	NUTUtil::GetUnitTestClassDefList(UnitTestClassDefaults);

	// Sort the list, based on type and implementation date, before adding
	NUTUtil::SortUnitTestClassDefList(UnitTestClassDefaults);


	for (auto CurDef : UnitTestClassDefaults)
	{
		FString CurType = CurDef->GetUnitTestType();

		if (!CurType.IsEmpty())
		{
			OutBeautifiedNames.Add(FString::Printf(TEXT("%s.%s"), *CurType, *CurDef->GetUnitTestName()));
		}
		else
		{
			OutBeautifiedNames.Add(CurDef->GetUnitTestName());
		}

		OutTestCommands.Add(CurDef->GetUnitTestName());
	}
}

bool FAllUnitTestsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FAutomationConsoleCommand(FString::Printf(TEXT("UnitTest %s"), *Parameters)));
	ADD_LATENT_AUTOMATION_COMMAND(FNUTWaitForUnitTests());

	return true;
}

