// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/LogWidgetCommands.h"

#define LOCTEXT_NAMESPACE "LogWidgetCommands"

void FLogWidgetCommands::RegisterCommands()
{
	UI_COMMAND(CopyLogLines, "Copy", "Copies the selected log lines to the clipboard", EUserInterfaceActionType::Button,
				FInputChord(EModifierKey::Control, EKeys::C));

	UI_COMMAND(FindLogText, "Find", "Find text within the current log window tab", EUserInterfaceActionType::Button,
				FInputChord(EModifierKey::Control, EKeys::F));
}

#undef LOCTEXT_NAMESPACE
