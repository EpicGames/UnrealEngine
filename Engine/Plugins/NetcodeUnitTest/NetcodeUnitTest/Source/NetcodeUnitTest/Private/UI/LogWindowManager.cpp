// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/LogWindowManager.h"
#include "Framework/Application/SlateApplication.h"


#include "NetcodeUnitTest.h"
#include "UnitTestManager.h"

#include "UI/SLogWindow.h"

/**
 * FLogWindowManager
 */

FLogWindowManager::~FLogWindowManager()
{
	// Remove all log window hooks
	for (int i=0; i<GridSpaces.Num(); i++)
	{
		FLogGridEntry* CurEntry = &GridSpaces[i];

		if (CurEntry->LogWindow.IsValid())
		{
			CurEntry->LogWindow->MultiOnWindowClosed.Remove(OnWindowClosedDelegateHandles.FindRef(CurEntry->LogWindow.Get()));
		}
	}

	for (int i=0; i<OverflowWindows.Num(); i++)
	{
		if (OverflowWindows[i].IsValid())
		{
			OverflowWindows[i]->MultiOnWindowClosed.Remove(OnWindowClosedDelegateHandles.FindRef(OverflowWindows[i].Get()));
		}
	}
}

void FLogWindowManager::Initialize(int InLogWidth, int InLogHeight)
{
	if (!bInitialized)
	{
		bInitialized = true;
		LogWidth = InLogWidth;
		LogHeight = InLogHeight;

		FDisplayMetrics DispMetrics;
		FSlateApplication::Get().GetDisplayMetrics(DispMetrics);


		// Using the grabbed display metrics, and the log window dimensions, setup the grid entries
		FPlatformRect WorkAreaRect = DispMetrics.PrimaryDisplayWorkAreaRect;

		// Calculate how many windows will fit horizontally, and then vertically
		int HorizontalCount = FMath::FloorToInt(static_cast<float>((WorkAreaRect.Right - WorkAreaRect.Left) / LogWidth));
		int VerticalCount = FMath::FloorToInt(static_cast<float>((WorkAreaRect.Bottom - WorkAreaRect.Top) / LogHeight));

		// Now setup the grid
		for (int VIdx = 0; VIdx<VerticalCount; VIdx++)
		{
			for (int HIdx = 0; HIdx<HorizontalCount; HIdx++)
			{
				FLogGridEntry CurEntry;

				// Calculate the current offset of the log window, for this grid entry
				int VerticalOffset = LogHeight * VIdx;
				int HorizontalOffset = LogWidth * HIdx;

				CurEntry.Top = static_cast<float>(WorkAreaRect.Top + VerticalOffset);
				CurEntry.Bottom = static_cast<float>(WorkAreaRect.Top + VerticalOffset + LogHeight);
				CurEntry.Left = static_cast<float>(WorkAreaRect.Left + HorizontalOffset);
				CurEntry.Right = static_cast<float>(WorkAreaRect.Left + HorizontalOffset + LogWidth);

				GridSpaces.Add(CurEntry);
			}
		}


		// @todo #JohnBDebug: Remove this debug code
#if 0
		for (int i = 0; i<GridSpaces.Num(); i++)
		{
			UE_LOGF(LogUnitTest, Log, "Grid entry '%i': Top: %f, Bottom: %f, Left: %f, Right: %f", i, GridSpaces[i].Top,
				GridSpaces[i].Bottom, GridSpaces[i].Left, GridSpaces[i].Right);
		}
#endif
	}
}

TSharedPtr<SLogWindow> FLogWindowManager::CreateLogWindow(FString Title, ELogType ExpectedFilters, bool bStatusWindow)
{
	TSharedPtr<SLogWindow> ReturnVal = nullptr;

	// Instead of searching for the first empty space, try to create log windows in a row pattern based upon the grid
	int FreeGridPos = FindFreeGridPos();
	bool bMinimizeWindow = false;


	// If there is a free position, create the log window
	if (FreeGridPos != INDEX_NONE)
	{
		LastLogWindowPos = FreeGridPos;
		FLogGridEntry& CurEntry = GridSpaces[FreeGridPos];

		ReturnVal =
				SNew(SLogWindow, Title, CurEntry.Left, CurEntry.Top, static_cast<float>(LogWidth), static_cast<float>(LogHeight))
				.bStatusWindow(bStatusWindow)
				.ExpectedFilters(ExpectedFilters);

		CurEntry.LogWindow = ReturnVal;
	}
	// If there isn't a free position, add to the overflow window array, and start the window minimized
	else
	{
		UNIT_ASSERT(GridSpaces.Num() > 0);

		// All minimized entries go into the first grid location
		FLogGridEntry& OverflowLoc = GridSpaces[0];

		ReturnVal =
				SNew(SLogWindow, Title, OverflowLoc.Left, OverflowLoc.Top, static_cast<float>(LogWidth), static_cast<float>(LogHeight))
				.bStatusWindow(bStatusWindow)
				.ExpectedFilters(ExpectedFilters);

		bMinimizeWindow = true;

		OverflowWindows.Add(ReturnVal);
	}


	if (ReturnVal.IsValid())
	{
		OnWindowClosedDelegateHandles.Add(ReturnVal.Get(),
											ReturnVal->MultiOnWindowClosed.AddRaw(this, &FLogWindowManager::OnWindowClosed));

		FSlateApplication::Get().AddWindow(ReturnVal.ToSharedRef());

		if (bMinimizeWindow)
		{
			ReturnVal->Minimize();
		}
		// Flash all newly focused windows, so it's clear the window is new (since it may take the place of an old window)
		else
		{
			ReturnVal->FlashWindow();
		}
	}

	return ReturnVal;
}

void FLogWindowManager::OnWindowClosed(const TSharedRef<SWindow>& ClosedWindow)
{
	TSharedPtr<SLogWindow> ClosedPtr = StaticCastSharedRef<SLogWindow>(ClosedWindow);

	if (ClosedPtr.IsValid())
	{
		// Clear the closed window, from the grid
		int FreeGridPos = INDEX_NONE;

		for (int i=0; i<GridSpaces.Num(); i++)
		{
			if (GridSpaces[i].LogWindow == ClosedPtr)
			{
				FreeGridPos = i;
				GridSpaces[i].LogWindow = nullptr;

				break;
			}
		}

		// If it was an overflow window, remove from the list
		OverflowWindows.Remove(ClosedPtr);


		// If there are any overflow windows waiting to be added to the grid, add one if a grid entry was freed
		if (FreeGridPos != INDEX_NONE && OverflowWindows.Num() > 0)
		{
			// First, find an overflow window which is still minimized - any that are not minimized
			// (detected by checking if the window was moved), assume the user has moved intentionally,
			// and thus should not be moved/touched again by this code
			int OverflowIdx = INDEX_NONE;

			for (int i=0; i<OverflowWindows.Num(); i++)
			{
				if (!OverflowWindows[i]->bHasMoved)
				{
					OverflowIdx = i;
					break;
				}
			}


			// If there are any user-untouched overflow windows, stick them into the grid slot
			if (OverflowIdx != INDEX_NONE)
			{
				FLogGridEntry& CurEntry = GridSpaces[FreeGridPos];

				CurEntry.LogWindow = OverflowWindows[OverflowIdx];
				OverflowWindows.RemoveAt(OverflowIdx);

				CurEntry.LogWindow->BringToFront();
				CurEntry.LogWindow->MoveWindowTo(FVector2D(CurEntry.Left, CurEntry.Top));

				// Flash the window now that it's focused, so it's clear that the window is newly placed (i.e. is not the old window)
				CurEntry.LogWindow->FlashWindow();
			}
		}


		// Notify the unit test manager, in case action is to be taken upon closing a log window
		if (GUnitTestManager != nullptr)
		{
			GUnitTestManager->NotifyLogWindowClosed(ClosedWindow);
		}
	}
}

int FLogWindowManager::FindFreeGridPos()
{
	int ReturnVal = INDEX_NONE;

	for (int i=0; i<GridSpaces.Num(); i++)
	{
		int CurIdx = LastLogWindowPos + i + 1;

		// Wraparound to the beginning
		if (CurIdx >= GridSpaces.Num())
		{
			CurIdx -= GridSpaces.Num();
		}

		if (!GridSpaces[CurIdx].LogWindow.IsValid())
		{
			ReturnVal = CurIdx;
			break;
		}
	}

	return ReturnVal;
}
