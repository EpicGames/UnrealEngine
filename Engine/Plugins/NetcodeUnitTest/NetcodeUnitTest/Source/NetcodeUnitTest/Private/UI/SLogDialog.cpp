// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SLogDialog.h"
#include "Widgets/SWindow.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "NetcodeUnitTest.h"


#define LOCTEXT_NAMESPACE "Dialogs"

// Largely a carbon copy, of UnrealEd Dialogs.cpp


class SLogChoiceDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLogChoiceDialog)
	{
	}
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(FText, Message)	
		SLATE_ATTRIBUTE(float, WrapMessageAt)
		SLATE_ATTRIBUTE(EAppMsgType::Type, MessageType)
	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

	void Construct(const FArguments& InArgs)
	{
		ParentWindow = InArgs._ParentWindow.Get();
		ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));
		Response = EAppReturnType::Cancel;

		FSlateFontInfo MessageFont( FCoreStyle::Get().GetFontStyle("StandardDialog.LargeFont"));
		MyMessage = InArgs._Message;

		TSharedPtr<SUniformGridPanel> ButtonBox;

		this->ChildSlot
			[	
				SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)

						+SVerticalBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.FillHeight(1.0f)
							.MaxHeight(550)
							.Padding(12.f)
							[
								SNew(SScrollBox)

								+ SScrollBox::Slot()
									[
										SNew(STextBlock)
											.Text(MyMessage)
											.Font(MessageFont)
											.WrapTextAt(InArgs._WrapMessageAt)
									]
							]

						+SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Bottom)
							.Padding(12.f, 2.f)
							[
								SAssignNew(ButtonBox, SUniformGridPanel)
									.SlotPadding(FCoreStyle::Get().GetMargin("StandardDialog.SlotPadding"))
									.MinDesiredSlotWidth(FCoreStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
									.MinDesiredSlotHeight(FCoreStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
							]
					]
			];

		int32 SlotIndex = 0;

#define ADD_SLOT(Button) \
		ButtonBox->AddSlot(SlotIndex++, 0) \
		[ \
			SNew(SButton) \
				.Text(EAppReturnTypeToText(EAppReturnType::Button)) \
				.OnClicked(this, &SLogChoiceDialog::HandleButtonClicked, EAppReturnType::Button) \
				.ContentPadding(FCoreStyle::Get().GetMargin("StandardDialog.ContentPadding")) \
				.HAlign(HAlign_Center) \
		];

		switch (InArgs._MessageType.Get())
		{	
		case EAppMsgType::Ok:
			ADD_SLOT(Ok)
			break;

		case EAppMsgType::YesNo:
			ADD_SLOT(Yes)
			ADD_SLOT(No)
			break;

		case EAppMsgType::OkCancel:
			ADD_SLOT(Ok)
			ADD_SLOT(Cancel)
			break;

		case EAppMsgType::YesNoCancel:
			ADD_SLOT(Yes)
			ADD_SLOT(No)
			ADD_SLOT(Cancel)
			break;

		case EAppMsgType::CancelRetryContinue:
			ADD_SLOT(Cancel)
			ADD_SLOT(Retry)
			ADD_SLOT(Continue)
			break;

		case EAppMsgType::YesNoYesAllNoAll:
			ADD_SLOT(Yes)
			ADD_SLOT(No)
			ADD_SLOT(YesAll)
			ADD_SLOT(NoAll)
			break;

		case EAppMsgType::YesNoYesAllNoAllCancel:
			ADD_SLOT(Yes)
			ADD_SLOT(No)
			ADD_SLOT(YesAll)
			ADD_SLOT(NoAll)
			ADD_SLOT(Cancel)
			break;

		case EAppMsgType::YesNoYesAll:
			ADD_SLOT(Yes)
			ADD_SLOT(No)
			ADD_SLOT(YesAll)
			break;

		default:
			UE_LOGF(LogUnitTest, Fatal, "Invalid Message Type");
		}

#undef ADD_SLOT
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


	EAppReturnType::Type GetResponse()
	{
		return Response;
	}

	virtual	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		// see if we pressed the Enter or Spacebar keys
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return HandleButtonClicked(EAppReturnType::Cancel);
		}

		// if it was some other button, ignore it
		return FReply::Unhandled();
	}

	void OnWindowClosed(const TSharedRef<SWindow>& ClosedWindow)
	{
		FOnLogDialogResult CachedCallback = ResultCallback;
		ResultCallback.Unbind();

		CachedCallback.ExecuteIfBound(ParentWindow.ToSharedRef(), Response, true);
	}

	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/** Converts an EAppReturnType into a localized FText */
	static FText EAppReturnTypeToText(EAppReturnType::Type ReturnType)
	{
		switch(ReturnType)
		{
		case EAppReturnType::No:
			return LOCTEXT("EAppReturnTypeNo", "No");

		case EAppReturnType::Yes:
			return LOCTEXT("EAppReturnTypeYes", "Yes");

		case EAppReturnType::YesAll:
			return LOCTEXT("EAppReturnTypeYesAll", "Yes All");

		case EAppReturnType::NoAll:
			return LOCTEXT("EAppReturnTypeNoAll", "No All");

		case EAppReturnType::Cancel:
			return LOCTEXT("EAppReturnTypeCancel", "Cancel");

		case EAppReturnType::Ok:
			return LOCTEXT("EAppReturnTypeOk", "OK");

		case EAppReturnType::Retry:
			return LOCTEXT("EAppReturnTypeRetry", "Retry");

		case EAppReturnType::Continue:
			return LOCTEXT("EAppReturnTypeContinue", "Continue");

		default:
			return LOCTEXT("MissingType", "MISSING RETURN TYPE");
		}
	}

private:
	// Handles clicking a message box button.
	FReply HandleButtonClicked(EAppReturnType::Type InResponse)
	{
		Response = InResponse;

		FOnLogDialogResult CachedCallback = ResultCallback;
		ResultCallback.Unbind();

		CachedCallback.ExecuteIfBound(ParentWindow.ToSharedRef(), Response, false);

		ParentWindow->RequestDestroyWindow();

		return FReply::Handled();
	}

		
public:
	/** Callback delegate that is triggered, when the dialog is run in non-modal mode */
	FOnLogDialogResult ResultCallback;

private:
	EAppReturnType::Type Response;
	TSharedPtr<SWindow> ParentWindow;
	TAttribute<FText> MyMessage;
};


static void CreateLogDialogWindow(TSharedPtr<SWindow>& OutWindow, TSharedPtr<SLogChoiceDialog>& OutDialog,
									EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle,
									FOnLogDialogResult ResultCallback=nullptr)
{
	OutWindow = SNew(SWindow)
		.Title(InTitle)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false).SupportsMaximize(false);

	OutDialog = SNew(SLogChoiceDialog)
		.ParentWindow(OutWindow)
		.Message(InMessage)
		.WrapMessageAt(512.0f)
		.MessageType(InMessageType);

	OutDialog->ResultCallback = ResultCallback;

	OutWindow->SetContent(OutDialog.ToSharedRef());
	OutWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(OutDialog.Get(), &SLogChoiceDialog::OnWindowClosed));
}

TSharedRef<SWindow> OpenLogDialog_NonModal(EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle,
											FOnLogDialogResult ResultCallback/*=nullptr*/)
{
	TSharedPtr<SWindow> MsgWindow = nullptr;
	TSharedPtr<SLogChoiceDialog> MsgDialog = nullptr;

	CreateLogDialogWindow(MsgWindow, MsgDialog, InMessageType, InMessage, InTitle, ResultCallback);

	FSlateApplication::Get().AddWindow(MsgWindow.ToSharedRef());

	return MsgWindow.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
