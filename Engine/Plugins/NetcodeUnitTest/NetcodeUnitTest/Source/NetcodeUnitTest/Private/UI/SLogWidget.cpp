// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SLogWidget.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Layout/SSpacer.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SCheckBox.h"
#include "ProcessUnitTest.h"
#include "Widgets/Docking/SDockTab.h"
#include "UI/LogWidgetCommands.h"
#include "NUTUtil.h"
#include "HAL/PlatformApplicationMisc.h"

#include "UI/SMultiSelectTableRow.h"


// Enable access to the private SEditableTextBox.EditableText variable, using the GET_PRIVATE macro
IMPLEMENT_GET_PRIVATE_VAR(SEditableTextBox, EditableText, TSharedPtr<SEditableText>);

// Enable access to SButton.Style
IMPLEMENT_GET_PRIVATE_VAR(SButton, Style, const FButtonStyle*);

// Enable reading SBorder::GetBorderBackgroundColor
IMPLEMENT_GET_PROTECTED_FUNC_CONST(SBorder, GetBorderBackgroundColor, FSlateColor, void,, const);

// Enable access to SDockTab::GetCurrentStyle, using the CALL_PROTECTED macro
IMPLEMENT_GET_PROTECTED_FUNC_CONST(SDockTab, GetCurrentStyle, const FDockTabStyle&, void,, const);


/**
 * Delegate used for recursively iterating a widgets child widgets, and testing if they match a search condition
 *
 * @param InWidget	The widget to be tested
 * @return			Whether or not the widget matches a search condition
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnTestWidget, const TSharedRef<SWidget>& /* InWidget */);


/**
 * Iteratively searches through a widget and all its child widgets, for a-widget/widgets, that pass the test delegate conditions
 *
 * @param OutMatches	The array of widgets that results are passed to
 * @param InWidget		The widget to search
 * @param InTester		The delegate (usually a lambda) to do condition testing with
 * @param bMultiMatch	(Internal) Whether or not multiple matches are searched for (or just one single widget, the first match)
 * @return				Whether or not any widgets were found
 */
static bool SearchForWidgets(TArray<TSharedRef<SWidget>>& OutMatches, TSharedRef<SWidget> InWidget, FOnTestWidget InTester,
						bool bMultiMatch=true)
{
	bool bFoundWidget = false;

	if (InTester.Execute(InWidget))
	{
		OutMatches.Add(InWidget);
		bFoundWidget = true;
	}

	if (bMultiMatch || !bFoundWidget)
	{
		FChildren* ChildWidgets = InWidget->GetChildren();
		int32 ChildCount = ChildWidgets->Num();

		for (int32 i=0; i<ChildCount; i++)
		{
			TSharedRef<SWidget> CurChild = ChildWidgets->GetChildAt(i);

			bFoundWidget = SearchForWidgets(OutMatches, CurChild, InTester, bMultiMatch) || bFoundWidget;

			if (!bMultiMatch && bFoundWidget)
			{
				break;
			}
		}
	}

	return bFoundWidget;
}

/**
 * Iteratively searches through a widget and all its child widgets, for a widget, that passes the test delegate conditions
 *
 * @param InWidget		The widget to search
 * @param InTester		The delegate (usually a lambda) to do condition testing with
 * @return				The returned widget (or SNullWidget, if none)
 */
static TSharedRef<SWidget> SearchForWidget(TSharedRef<SWidget> InWidget, FOnTestWidget InTester)
{
	TSharedRef<SWidget> ReturnVal = SNullWidget::NullWidget;
	TArray<TSharedRef<SWidget>> Match;

	if (SearchForWidgets(Match, InWidget, InTester, false))
	{
		ReturnVal = Match[0];
	}

	return ReturnVal;
}

/**
 * Special tab type, that cannot be dragged/undocked from the tab bar
 */
class SLockedTab : public SDockTab
{
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled();
	}
};


/**
 * SLogWidget
 */

void SLogWidget::Construct(const FArguments& Args)
{
	LogWidgetCommands = MakeShareable(new FUICommandList);

	// Setup the context-menu/shortcut-key commands
	const FLogWidgetCommands& Commands = FLogWidgetCommands::Get();

	LogWidgetCommands->MapAction(Commands.CopyLogLines, FExecuteAction::CreateRaw(this, &SLogWidget::OnCopy),
									FCanExecuteAction::CreateRaw(this, &SLogWidget::CanCopy));

	LogWidgetCommands->MapAction(Commands.FindLogText, FExecuteAction::CreateRaw(this, &SLogWidget::OnFind),
									FCanExecuteAction::CreateRaw(this, &SLogWidget::CanFind));


	// Setup the tabbed log filters layout
	TSharedRef<FTabManager::FLayout> LogTabLayout = InitializeTabLayout(Args);


	// Lists widgets that share the same tooltip text
	TArray<TSharedPtr<SWidget>> AutoCloseWidgets;
	TArray<TSharedPtr<SWidget>> AutoScrollWidgets;
	TArray<TSharedPtr<SWidget>> DeveloperWidgets;

	// Add a blank element to above array, and return a ref to that element - for tidiness/compatibility with Slate declarative syntax
	auto ArrayAddNew =
		[] (TArray<TSharedPtr<SWidget>>& InArray) -> TSharedPtr<SWidget>&
		{
			int NewIndex = InArray.Add(NULL);
			return InArray[NewIndex];
		};


	// Conditionally enable/disable some UI elements (e.g. Suspend/AutoClose), that aren't necessary for the status window
	auto ConditionalSlot =
		[] (bool bCondition, SHorizontalBox::FSlot::FSlotArguments& InSlot) -> SHorizontalBox::FSlot::FSlotArguments&
		{
			if (!bCondition)
			{
				InSlot.AttachWidget(SNullWidget::NullWidget);
			}

			return InSlot;
		};


	// Setup the child widgets
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.Padding(2.0f, 2.0f)
			[
				/**
				 * Log list filter tabs
				 */
				LogTabManager->RestoreFrom(LogTabLayout, TSharedPtr<SWindow>()).ToSharedRef()
			]
		+SVerticalBox::Slot()
			.VAlign(VAlign_Bottom)
			.Padding(2.0f, 2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				/**
				 * Search button
				 */
				+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
						.ToolTipText(FText::FromString(TEXT("Open the find bar for the current tab.")))
						.OnClicked_Lambda(
							[&]()
						{
							OnFind();
							return FReply::Handled();
						})
						[
							SNew(SImage)
							.Image(&FCoreStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox").GlassImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]

				/**
				 * Suspend/Resume button
				 */
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
						.ToolTipText(FText::FromString(
							TEXT("Suspend/resume the server process, so that a debugger can be manually attached.")))
						.OnClicked_Lambda(
							[&]()
							{
								OnSuspendClicked.ExecuteIfBound();
								return FReply::Handled();
							})
						.Content()
						[
							SAssignNew(SuspendButtonText, STextBlock)
							.Text(FText::FromString(FString(TEXT("SUSPEND"))))
						]
					]
				)
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SSpacer)
					]
				)


				/**
				 * AutoClose checkbox
				 */
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ArrayAddNew(AutoCloseWidgets), STextBlock)
						.Text(FText::FromString(FString(TEXT("AutoClose:"))))
					]
				)
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ArrayAddNew(AutoCloseWidgets), SCheckBox)
						.IsChecked((bAutoClose ? ECheckBoxState::Checked : ECheckBoxState::Unchecked))
						.OnCheckStateChanged_Lambda(
						[&](ECheckBoxState NewAutoCloseState)
						{
							bAutoClose = (NewAutoCloseState == ECheckBoxState::Checked);
						})
					]
				)
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SSpacer)
					]
				)


				/**
				 * AutoScroll checkbox
				 */
				+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ArrayAddNew(AutoScrollWidgets), STextBlock)
						.Text(FText::FromString(FString(TEXT("AutoScroll:"))))
					]
				+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ArrayAddNew(AutoScrollWidgets), SCheckBox)
						// Disable the checkbox, if the current tab doesn't support autoscrolling
						.IsEnabled_Lambda(
							[&]()
							{
								return CanAutoScroll(GetActiveTabInfo().ToSharedRef());
							})
						.IsChecked(ECheckBoxState::Checked)
						.OnCheckStateChanged_Lambda(
							[&](ECheckBoxState NewAutoScrollState)
							{
								bAutoScroll = (NewAutoScrollState == ECheckBoxState::Checked);

								// Scroll to the end, if autoscroll was just re-enabled
								if (bAutoScroll)
								{
									for (auto CurTabInfo : LogTabs)
									{
										if (CanAutoScroll(CurTabInfo))
										{
											ScrollToEnd(CurTabInfo);
										}
									}
								}
							})
					]


				/**
				 * Developer checkbox
				 */
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SSpacer)
					]
				)
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ArrayAddNew(DeveloperWidgets), STextBlock)
						.Text(FText::FromString(FString(TEXT("Developer:"))))
					]
				)
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ArrayAddNew(DeveloperWidgets), SCheckBox)
						.IsChecked(ECheckBoxState::Unchecked)
						.OnCheckStateChanged_Lambda(
							[&](ECheckBoxState NewDeveloperState)
							{
								OnDeveloperClicked.ExecuteIfBound(NewDeveloperState == ECheckBoxState::Checked);
							})
					]
				)


				/**
				 * Console command context selector
				 */
				+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(16.f, 0.f))
					]
				+ConditionalSlot(!Args._bStatusWidget,
					SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ConsoleComboBox, SComboBox<TSharedPtr<FString>>)
						.OptionsSource(&ConsoleContextList)
						.ToolTipText(FText::FromString(FString(TEXT("Select the context for executing console commands."))))
						.OnGenerateWidget_Lambda(
								[](TSharedPtr<FString> Item)
								{
									FString ItemStr = *Item;
									FString ToolTipStr = TEXT("");

									if (ItemStr == TEXT("Global"))
									{
										ToolTipStr = TEXT("Execute the command outside the context of any unit test world.");
									}
									else if (ItemStr == TEXT("Local"))
									{
										ToolTipStr = TEXT("Execute the command on the local-client/unit-test.");
									}
									else if (ItemStr == TEXT("Server"))
									{
										ToolTipStr = TEXT("Execute the command on the game server associated with this unit test.");
									}
									else if (ItemStr == TEXT("Client"))
									{
										// @todo #JohnBFeatureUI: Update when implemented
										ToolTipStr = TEXT("(Not yet implemented) ") 
														TEXT("Execute the command on the client associated with this unit test.");
									}

									// @todo #JohnB: Custom context hints?

									return	SNew(STextBlock)
											.Text(FText::FromString(ItemStr))
											.ToolTipText(FText::FromString(ToolTipStr));
								}
							)
							[
								SNew(STextBlock)
								.Text_Lambda(
									[&]()
									{
										TSharedPtr<FString> Selection = ConsoleComboBox->GetSelectedItem();

										return FText::FromString(Selection.IsValid() ? *Selection : DefaultConsoleContext);
									}
								)
							]
					]
				)

				/**
				 * Console command edit box
				 */
				 // @todo #JohnBFeatureUI: Borrow the auto-complete from SOutputLog's version of log windows
				+SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 0.0f)
					[
						SAssignNew(ConsoleTextBox, SEditableTextBox)
						.HintText(FText::FromString(TEXT("Console")))
						.ToolTipText(FText::FromString(TEXT("Executes a console command within the specified context.")))
						.ClearKeyboardFocusOnCommit(false)
						.OnTextCommitted_Lambda(
							[&](const FText& InText, ETextCommit::Type InCommitType)
							{
								if (OnConsoleCommand.IsBound())
								{
									FString Command = InText.ToString();

									if (InCommitType == ETextCommit::OnEnter && Command.Len() > 0)
									{
										TSharedPtr<FString> ComboSelection =
											(ConsoleComboBox.IsValid() ? ConsoleComboBox->GetSelectedItem() : NULL);

										FString CommandContext = (ComboSelection.IsValid() ? *ComboSelection : DefaultConsoleContext);

										bool bSuccess = OnConsoleCommand.Execute(CommandContext, Command);

										// If the command executed successfully, wipe the text
										if (bSuccess)
										{
											ConsoleTextBox->SetText(FText());
										}
										// Otherwise, do 'select-all' on the text (user can still amend mistakes, or wipe by typing)
										else
										{
											// ConsoleTextBox->SelectAllText();
											(GET_PRIVATE(SEditableTextBox, ConsoleTextBox, EditableText))->SelectAllText();
										}
									}
								}
							})
						]
			]
	];



	// Set widget tooltip text, for UI self-documentation
	for (auto CurWidget : AutoCloseWidgets)
	{
		CurWidget->SetToolTipText(
			FText::FromString(TEXT("Whether or not to automatically close this window, when the unit test completes.")));
	}

	for (auto CurWidget : AutoScrollWidgets)
	{
		CurWidget->SetToolTipText(
			FText::FromString(TEXT("Whether or not to automatically scroll to the bottom, as new log entries arrive.")));
	}

	for (auto CurWidget : DeveloperWidgets)
	{
		CurWidget->SetToolTipText(
			FText::FromString(TEXT("Whether or not to use developer mode (keeps the unit test and any server/client from closing.)")));
	}
}

TSharedRef<FTabManager::FLayout> SLogWidget::InitializeTabLayout(const FArguments& Args)
{
	// Initialize the LogTabs array (which includes labels/tooltips, for each log type)
	LogTabs.Add(MakeShareable(new
		FLogTabInfo(TEXT("Summary"),	TEXT("Filter for the most notable log entries."), ELogType::StatusImportant, 10)));

	if (Args._bStatusWidget)
	{
		LogTabs.Add(MakeShareable(new
			FLogTabInfo(TEXT("Advanced Summary"),	TEXT("Filter for the most notable log entries, with extra/advanced information."),
						ELogType::StatusImportant | ELogType::StatusVerbose | ELogType::StatusAdvanced, 20)));
	}

	LogTabs.Add(MakeShareable(new
		FLogTabInfo(TEXT("All"),		TEXT("No filters - all log output it shown."), ELogType::All, 30)));

	if (!Args._bStatusWidget)
	{
		LogTabs.Add(MakeShareable(new
			FLogTabInfo(TEXT("Local"),	TEXT("Filter for locally-sourced log entries (i.e. no sub-process logs)."), ELogType::Local)));

		if (!!(Args._ExpectedFilters & ELogType::Server))
		{
			LogTabs.Add(MakeShareable(new
				FLogTabInfo(TEXT("Server"),		TEXT("Filter for the server process log entries."), ELogType::Server)));
		}

		if (!!(Args._ExpectedFilters & ELogType::Client))
		{
			LogTabs.Add(MakeShareable(new
				FLogTabInfo(TEXT("Client"),		TEXT("Filter for the client process log entries."), ELogType::Client)));
		}

		// Always spawn the debug tab, even if not expecting debug messages, but if not expecting,
		// then keep it closed unless a debug message turns up
		bool bOpenDebugTab = ((Args._ExpectedFilters & ELogType::StatusDebug) == ELogType::StatusDebug);

		LogTabs.Add(MakeShareable(new
			FLogTabInfo(TEXT("Debug"),		TEXT("Filter for debug log entries."), ELogType::StatusDebug, 5, bOpenDebugTab)));
	}

	LogTabs.Add(MakeShareable(new
		FLogTabInfo(TEXT("Console"), TEXT("Filter for local console command results."), ELogType::OriginConsole, 5, false)));


	// Initialize the tab manager, stack and layout
	TSharedRef<SDockTab> DudTab = SNew(SLockedTab);
	LogTabManager = FGlobalTabmanager::Get()->NewTabManager(DudTab);


	// Add tabs for each LogTabs entry, to the tab stack
	const TSharedRef<FTabManager::FStack> LogTabStack = FTabManager::NewStack();

	for (auto CurTabInfo : LogTabs)
	{
		LogTabManager->RegisterTabSpawner(CurTabInfo->TabIdName, FOnSpawnTab::CreateRaw(this, &SLogWidget::SpawnLogTab));

		LogTabStack->AddTab(CurTabInfo->TabIdName, (CurTabInfo->bTabOpen ? ETabState::OpenedTab : ETabState::ClosedTab));
	}

	// Make the first tab in the stack, the foremost tab
	LogTabStack->SetForegroundTab(LogTabs[0]->TabIdName);


	// Setup the layout and add the stack; the layout internally handles how the tab widgets are laid out, upon creation (happens later)
	TSharedRef<FTabManager::FLayout> LogTabLayout =
		FTabManager::NewLayout("NetcodeUnitTestLogTabLayout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				LogTabStack
			)
		);

	return LogTabLayout;
}

TSharedRef<SDockTab> SLogWidget::SpawnLogTab(const FSpawnTabArgs& InSpawnTabArgs)
{
	FName CurTabName = InSpawnTabArgs.GetTabId().TabType;

	TSharedRef<FLogTabInfo> CurTabInfo = *LogTabs.FindByPredicate(
		[&CurTabName](const TSharedRef<FLogTabInfo>& CurElement)
		{
			return CurElement->TabIdName == CurTabName;
		});

	auto ArrayAddNew =
		[] (TArray<TSharedPtr<SWidget>>& InArray) -> TSharedPtr<SWidget>&
		{
			int NewIndex = InArray.Add(NULL);
			return InArray[NewIndex];
		};

	// Have the 'find bar', use the same close button style as tabs
	const FDockTabStyle* const TabStyle = &FCoreStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab");
	const FButtonStyle* const CloseButtonStyle = &TabStyle->CloseButtonStyle;

	TSharedRef<SDockTab> ReturnVal = SNew(SLockedTab)
			.TabRole(ETabRole::MajorTab)
			.Label(FText::FromString(CurTabInfo->Label))
			// For tabs, must specify a ToolTip parameter - can't use ToolTipText, as the internal tab code ignores it (a bug)
			.ToolTip(
				SNew(SToolTip)
				.Text(FText::FromString(CurTabInfo->ToolTip))
			)
			// Can't close these tabs
			.OnCanCloseTab_Lambda(
				[]()
				{
					return false;
				}
			)
			.TabWellContentLeft()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(FString(TEXT("Filter:"))))
						.ToolTipText(FText::FromString(FString(TEXT("The type of filtering to be applied to log output."))))
					]
			]
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					[
						/**
						 * Log list view
						 */
						SAssignNew(CurTabInfo->LogListView, SListView<TSharedRef<FLogLine>>)
						.ListItemsSource(&CurTabInfo->TabLogLines)
						.OnGenerateRow_Lambda(
							[](TSharedRef<FLogLine> Item, const TSharedRef<STableViewBase>& OwnerTable)
							{
								ELogType CurLogType = Item->LogType;

								// Various types of special font formatting
								const int32 FontSize = 9;
								FSlateFontInfo RenderFont = FCoreStyle::GetDefaultFontStyle("Regular", FontSize);
								if (!!(CurLogType & ELogType::StyleMonospace))
								{
									RenderFont = FCoreStyle::GetDefaultFontStyle("Mono", FontSize);
								}
								else if (!!(CurLogType & ELogType::StyleBold) && !!(CurLogType & ELogType::StyleItalic))
								{
									RenderFont = FCoreStyle::GetDefaultFontStyle("BoldCondensedItalic", FontSize);
								}
								else if (!!(CurLogType & ELogType::StyleBold))
								{
									RenderFont = FCoreStyle::GetDefaultFontStyle("Bold", FontSize);
								}
								else if (!!(CurLogType & ELogType::StyleItalic))
								{
									RenderFont = FCoreStyle::GetDefaultFontStyle("Italic", FontSize);
								}

								FString RenderText = *Item->LogLine;

								// Pseudo-underline; just adds newline, and then underlines with lots of ----
								if (!!(CurLogType & ELogType::StyleUnderline) && RenderText.Len() > 0)
								{
									const TCHAR* TextToUnderline = *RenderText;
									const TCHAR* UnderlineEnd = TextToUnderline + RenderText.Len()-1;
									uint32 TotalLen = RenderText.Len();

									// Only underline up to the last alphanumeric character
									while (UnderlineEnd > TextToUnderline && !FChar::IsAlnum(*UnderlineEnd))
									{
										UnderlineEnd--;
										TotalLen--;
									}


									const TSharedRef<FSlateFontMeasure> FontMeasure =
																		FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

									FVector2f UnderlineDim = FontMeasure->Measure(TextToUnderline, 0, TotalLen, RenderFont);
									FVector2f BaseDim = FontMeasure->Measure(TEXT("-"), RenderFont);

									int32 UnderlineCharCount = FMath::FloorToInt(UnderlineDim.X / BaseDim.X);

									if (UnderlineCharCount > 0)
									{
										RenderText += FString(TEXT("\r\n")) + FString::ChrN(UnderlineCharCount, '-');
									}
								}

								return SNew(SMultiSelectTableRow<TSharedRef<FString>>, OwnerTable)
										.Content()
										[
											SNew(STextBlock)
												.Text(FText::FromString(RenderText))
												.HighlightText(Item->LogHighlight.Len() > 0 ?
													FText::FromString(Item->LogHighlight) :
													TAttribute<FText>())
												.HighlightColor(FLinearColor(0.f, 1.f, 0.f, 0.25f))
												.Font(RenderFont)
												.ColorAndOpacity(Item->LogColor)
												.ToolTip(
													SNew(SToolTip)
													[
														// Apart from Text/Font, this is a copy-paste from SToolTip
														SNew(STextBlock)
														.Text(FText::FromString(RenderText))
														.Font(RenderFont)
														.ColorAndOpacity(FSlateColor::UseForeground())
														.WrapTextAt_Static(&SToolTip::GetToolTipWrapWidth)
													]
												)
										];
							})
						.OnContextMenuOpening_Lambda(
							[&]()
							{
								FMenuBuilder MenuBuilder(true, LogWidgetCommands);

								MenuBuilder.AddMenuEntry(FLogWidgetCommands::Get().CopyLogLines);
								MenuBuilder.AddMenuEntry(FLogWidgetCommands::Get().FindLogText);

								return MenuBuilder.MakeWidget();
							})
					]
				+SVerticalBox::Slot()
					.VAlign(VAlign_Bottom)
					.Padding(0.0f, 2.0f)
					.AutoHeight()
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						[
							SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), SBorder)
							.Visibility(EVisibility::Collapsed)
							.Padding(TabStyle->TabPadding)
							.BorderImage(&TabStyle->ForegroundBrush)
						]
						+SOverlay::Slot()
						[
							SNew(SHorizontalBox)

							// IMPORTANT: Do NOT use SSearchBox, as getting the style to use left-side buttons, can't be done inline
							//				(runtime tweaks or duplicating styles in general, appears to be completely impractical)

							/**
							 * 'Close Find Bar' 'x' button
							 */
							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), SButton)
									.Visibility(EVisibility::Collapsed)
									.ToolTipText(FText::FromString(TEXT("Close the find bar.")))
									.ButtonStyle(CloseButtonStyle)
									.ContentPadding(0.0f)
									.OnClicked_Lambda(
										[&]()
										{
											TSharedPtr<FLogTabInfo> ActiveTab = GetActiveTabInfo();

											for (TSharedPtr<SWidget>& CurWidget : ActiveTab->FindWidgets)
											{
												CurWidget->SetVisibility(EVisibility::Collapsed);
											}

											if (bAutoScroll)
											{
												ScrollToEnd(ActiveTab.ToSharedRef());
											}

											if (ActiveTab->bHighlightFind)
											{
												UpdateFindHighlight(ActiveTab, false);
											}
											else
											{
												ActiveTab->LogListView->RequestListRefresh();
											}

											return FReply::Handled();
										})
									[
										SNew(SSpacer)
										.Size(CloseButtonStyle->Normal.ImageSize)
									]
								]

							/**
							 * 'Find' label
							 */
							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), STextBlock)
									.Visibility(EVisibility::Collapsed)
									.Text(FText::FromString(TEXT("Find:")))
								]

							/**
							 * Find previous button
							 */
							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), SButton)
									.Visibility(EVisibility::Collapsed)
									.ToolTipText(FText::FromString(TEXT("Find the previous occurrence of the specified text.")))
									.Text(FText::FromString(TEXT("Prev")))
									.OnClicked_Lambda(
										[&]()
										{
											auto ActiveTabInfo = GetActiveTabInfo();
											ScrollToText(ActiveTabInfo.ToSharedRef(), ActiveTabInfo->FindBox->GetText().ToString(),
															true);

											return FReply::Handled();
										})
								]

							/**
							 * Find Next button
							 */
							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), SButton)
									.Visibility(EVisibility::Collapsed)
									.ToolTipText(FText::FromString(TEXT("Find the next occurrence of the specified text.")))
									.Text(FText::FromString(TEXT("Next")))
									.OnClicked_Lambda(
										[&]()
										{
											auto ActiveTabInfo = GetActiveTabInfo();
											ScrollToText(ActiveTabInfo.ToSharedRef(), ActiveTabInfo->FindBox->GetText().ToString());

											return FReply::Handled();
										})
								]

							/**
							 * Highlight checkbox
							 */
							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.f, 2.f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), STextBlock)
									.Visibility(EVisibility::Collapsed)
									.Text(FText::FromString(FString(TEXT("Highlight:"))))
								]
							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.f, 2.f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), SCheckBox)
									.Visibility(EVisibility::Collapsed)
									.IsChecked((CurTabInfo->bHighlightFind ? ECheckBoxState::Checked : ECheckBoxState::Unchecked))
									.OnCheckStateChanged_Lambda(
									[&](ECheckBoxState NewHighlightState)
									{
										TSharedPtr<FLogTabInfo> ActiveTab = GetActiveTabInfo();
										bool bHighlight = (NewHighlightState == ECheckBoxState::Checked);

										ActiveTab->bHighlightFind = bHighlight;

										UpdateFindHighlight(ActiveTab, bHighlight, ActiveTab->LastFind);
									})
								]

							+SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								.AutoWidth()
								[
									SAssignNew(ArrayAddNew(CurTabInfo->FindWidgets), SSpacer)
									.Visibility(EVisibility::Collapsed)
								]

							 +SHorizontalBox::Slot()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								[

									SNew(SOverlay)
									/**
									 * Find edit box
									 */
									+SOverlay::Slot()
									[
										SAssignNew(CurTabInfo->FindBox, SEditableTextBox)
										.Visibility(EVisibility::Collapsed)
										.HintText(FText::FromString(TEXT("Find")))
										.ToolTipText(FText::FromString(TEXT("Finds the specified text, within the current log tab.")))
										.ClearKeyboardFocusOnCommit(false)
										.OnTextCommitted_Lambda(
											[&](const FText& InText, ETextCommit::Type InCommitType)
											{
												if (InCommitType == ETextCommit::OnEnter)
												{
													auto ActiveTabInfo = GetActiveTabInfo();
													ScrollToText(ActiveTabInfo.ToSharedRef(), InText.ToString(),
																	ActiveTabInfo->bLastFindWasUp);
												}
											})
									]

									/**
									 * Find error label
									 */
									+SOverlay::Slot()
									.HAlign(HAlign_Right)
									[
										SNew(SHorizontalBox)
										+SHorizontalBox::Slot()
											.HAlign(HAlign_Right)
											.VAlign(VAlign_Center)
											.Padding(2.f, 2.f)
											.AutoWidth()
										[
											SAssignNew(CurTabInfo->FindErrorLabel, SBorder)
											.Visibility(EVisibility::Collapsed)
											.Padding(TabStyle->TabPadding)
											.BorderImage(&TabStyle->ForegroundBrush)
											.BorderBackgroundColor(FLinearColor(1.f, 1.f, 1.f, 1.f))
											[
												SAssignNew(CurTabInfo->FindErrorText, STextBlock)
												.ColorAndOpacity(FLinearColor(1.f, 0.5f, 0.f, 1.f))
											]
										]
									]
								]
						]
					]
			];

	CurTabInfo->FindWidgets.Add(CurTabInfo->FindBox);
	CurTabInfo->FindWidgets.Add(CurTabInfo->FindErrorLabel);
	CurTabInfo->FindWidgets.Add(CurTabInfo->FindErrorText);
	CurTabInfo->TabWidget = ReturnVal;

	// Disable the close button on the tab - a bit hacky, as need to 'manually' find the button (here it's identified by its style)
	TSharedRef<SWidget> CloseButton = SearchForWidget(ReturnVal, FOnTestWidget::CreateLambda
		(
			[&ReturnVal](const TSharedRef<SWidget>& InWidget)
			{
				bool bFound = false;

				if (InWidget->GetType() == FName(TEXT("SButton")))
				{
					TSharedRef<SButton> CurButton = StaticCastSharedRef<SButton>(InWidget);
					const FButtonStyle* ButtonStyle = GET_PRIVATE(SButton, CurButton, Style);
					const FDockTabStyle& CurTabStyle = CALL_PROTECTED(SDockTab, ReturnVal, GetCurrentStyle)();

					bFound = ButtonStyle == &CurTabStyle.CloseButtonStyle;
				}

				return bFound;
			}
		));

	// If the close button was found, hide it
	if (CloseButton != SNullWidget::NullWidget)
	{
		StaticCastSharedRef<SButton>(CloseButton)->SetVisibility(EVisibility::Hidden);
	}

	return ReturnVal;
}

TSharedPtr<FLogTabInfo> SLogWidget::GetActiveTabInfo() const
{
	TSharedPtr<FLogTabInfo> ReturnVal = NULL;

	for (auto CurTabInfo : LogTabs)
	{
		TSharedPtr<SDockTab> CurTab = CurTabInfo->TabWidget.Pin();

		if (CurTab.IsValid() && CurTab->IsForeground())
		{
			ReturnVal = CurTabInfo;
			break;
		}
	}

	return ReturnVal;
}


void SLogWidget::AddLine(ELogType InLogType, TSharedRef<FString> LogLine, FSlateColor LogColor/*=FSlateColor::UseForeground()*/,
							bool bTakeTabFocus/*=false*/)
{
	TSharedRef<FLogLine> CurLogEntry = MakeShareable(new FLogLine(InLogType, LogLine, LogColor));

	// Add the line to the main list
	LogLines.Add(CurLogEntry);

	TSharedPtr<FLogTabInfo> ActiveTab = GetActiveTabInfo();

	auto MatchesTabFilter =
		[&](const TSharedPtr<FLogTabInfo>& InTab)
		{
			return InTab->Filter == ELogType::All || !!(InTab->Filter & InLogType);
		};

	bool bLineInTabFocus = ActiveTab.IsValid() && MatchesTabFilter(ActiveTab);
	TSharedPtr<FLogTabInfo> FocusTab = nullptr;

	// Then add it to each log tab, if it passes that tabs filter
	for (TSharedRef<FLogTabInfo>& CurTabInfo : LogTabs)
	{
		if (MatchesTabFilter(CurTabInfo))
		{
			// If the tab is not presently open, open it now
			if (!CurTabInfo->bTabOpen && LogTabManager.IsValid())
			{
				LogTabManager->TryInvokeTab(CurTabInfo->TabIdName);

				// The new tab has stolen focus, now restore the old tabs focus
				LogTabManager->TryInvokeTab(ActiveTab->TabIdName);

				CurTabInfo->bTabOpen = true;
			}

			// If the line is requesting focus, but is not currently in focus, select a tab for focusing
			if (bTakeTabFocus && !bLineInTabFocus)
			{
				if (CurTabInfo != ActiveTab && (!FocusTab.IsValid() || CurTabInfo->Priority < FocusTab->Priority))
				{
					FocusTab = CurTabInfo;
				}
			}


			CurTabInfo->TabLogLines.Add(CurLogEntry);


			auto CurLogListView = CurTabInfo->LogListView;

			if (bAutoScroll && CanAutoScroll(CurTabInfo))
			{
				CurLogListView->RequestScrollIntoView(CurLogEntry);
			}

			FString HighlightFindStr = (CurTabInfo->bHighlightFind && CurTabInfo->FindBox->GetVisibility() != EVisibility::Collapsed) ?
										CurTabInfo->LastFind : TEXT("");

			if (HighlightFindStr.Len() > 0 && LogLine->Contains(HighlightFindStr))
			{
				UpdateFindHighlight(CurTabInfo, true, CurTabInfo->LastFind);
			}
			else
			{
				CurLogListView->RequestListRefresh();
			}
		}
	}

	// If a focus change is required, perform it
	if (FocusTab.IsValid() && LogTabManager.IsValid())
	{
		LogTabManager->TryInvokeTab(FocusTab->TabIdName);
	}
}


void SLogWidget::OnSuspendStateChanged(ESuspendState InSuspendState)
{
	if (SuspendButtonText.IsValid())
	{
		if (InSuspendState == ESuspendState::Active)
		{
			SuspendButtonText->SetText(NSLOCTEXT("SLogWidget", "SUSPEND", "SUSPEND"));
		}
		else if (InSuspendState == ESuspendState::Suspended)
		{
			SuspendButtonText->SetText(NSLOCTEXT("SLogWidget", "RESUME", "RESUME"));
		}
	}
}

FReply SLogWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply ReturnVal = FReply::Unhandled();

	// Pass the input event on to the widget command handler
	if (LogWidgetCommands.IsValid() && LogWidgetCommands->ProcessCommandBindings(InKeyEvent))
	{
		ReturnVal = FReply::Handled();
	}

	return ReturnVal;
}

void SLogWidget::OnCopy()
{
	TSharedPtr<FLogTabInfo> ActiveTabInfo = GetActiveTabInfo();

	if (ActiveTabInfo.IsValid())
	{
		TArray<TSharedRef<FLogLine>> SelectedLines = ActiveTabInfo->LogListView->GetSelectedItems();

		if (SelectedLines.Num())
		{
			// Make sure the selected items are sorted in descending order
			SelectedLines.Sort(
				[&ActiveTabInfo](const TSharedRef<FLogLine>& A, const TSharedRef<FLogLine>& B)
				{
					return ActiveTabInfo->TabLogLines.IndexOfByKey(A) < ActiveTabInfo->TabLogLines.IndexOfByKey(B);
				});


			FString CopiedLines;

			for (int i=0; i<SelectedLines.Num(); i++)
			{
				CopiedLines += SelectedLines[i]->LogLine.Get();

				if (i < SelectedLines.Num()-1)
				{
					CopiedLines += LINE_TERMINATOR;
				}
			}

			FPlatformApplicationMisc::ClipboardCopy(*CopiedLines);
		}
	}
}

bool SLogWidget::CanCopy() const
{
	bool bReturnVal = false;

	TSharedPtr<FLogTabInfo> ActiveTabInfo = GetActiveTabInfo();

	if (ActiveTabInfo.IsValid() && ActiveTabInfo->LogListView.IsValid() && ActiveTabInfo->LogListView->GetNumItemsSelected() > 0)
	{
		bReturnVal = true;
	}

	return bReturnVal;
}

void SLogWidget::OnFind()
{
	TSharedPtr<FLogTabInfo> ActiveTabInfo = GetActiveTabInfo();

	if (ActiveTabInfo.IsValid())
	{
		// Make the find bar visible
		for (auto CurFindWidget : ActiveTabInfo->FindWidgets)
		{
			if (CurFindWidget.IsValid())
			{
				CurFindWidget->SetVisibility(EVisibility::Visible);
			}
		}

		// Hide the search wrap-around label, by default
		if (ActiveTabInfo->FindErrorLabel->GetVisibility() != EVisibility::Collapsed)
		{
			ActiveTabInfo->FindErrorLabel->SetVisibility(EVisibility::Collapsed);
		}

		FString HighlightFindStr = ActiveTabInfo->bHighlightFind ? ActiveTabInfo->LastFind : TEXT("");

		if (HighlightFindStr.Len() > 0)
		{
			UpdateFindHighlight(ActiveTabInfo, true, ActiveTabInfo->LastFind);
		}

		FSlateApplication::Get().SetKeyboardFocus(ActiveTabInfo->FindBox);
	}
}

bool SLogWidget::CanFind() const
{
	bool bReturnVal = false;

	bReturnVal = GetActiveTabInfo().IsValid();

	return bReturnVal;
}

void SLogWidget::ScrollToEnd(TSharedRef<FLogTabInfo> InTab)
{
	auto CurLogListView = InTab->LogListView;
	auto& CurTabLogLines = InTab->TabLogLines;

	if (CurLogListView.IsValid() && CurTabLogLines.Num() > 0)
	{
		CurLogListView->RequestScrollIntoView(CurTabLogLines[CurTabLogLines.Num()-1]);
		CurLogListView->RequestListRefresh();
	}
}

void SLogWidget::ScrollToText(TSharedRef<FLogTabInfo> InTab, FString FindText, bool bSearchUp/*=false*/)
{
	auto CurLogListView = InTab->LogListView;
	auto& CurTabLogLines = InTab->TabLogLines;

	InTab->LastFind = FindText;

	if (CurLogListView.IsValid() && CurTabLogLines.Num() > 0)
	{
		int32 FindStartIdx = INDEX_NONE;
		TArray<TSharedRef<FLogLine>> SelectedLines = CurLogListView->GetSelectedItems();

		if (SelectedLines.Num() > 0)
		{
			FindStartIdx = CurTabLogLines.IndexOfByKey(SelectedLines[0]);
		}
		else if (bSearchUp)
		{
			FindStartIdx = CurTabLogLines.Num()-1;
		}
		else
		{
			FindStartIdx = 0;
		}


		int32 SearchDir = (bSearchUp ? -1 : 1);
		int32 FoundIdx = INDEX_NONE;
		bool bSearchWrapped = false;

		for (int32 i=FindStartIdx + SearchDir; true; i += SearchDir)
		{
			// When the start/end of the array is reached, wrap-around to the other end
			if (i < 0 || i >= CurTabLogLines.Num())
			{
				i = (bSearchUp ? CurTabLogLines.Num()-1 : 0);
				bSearchWrapped = true;
			}

			// Moved out of 'for' condition, and into the loop, to avoid infinite recursion in rare circumstances
			if (i == FindStartIdx)
			{
				break;
			}

			if (CurTabLogLines[i]->LogLine->Contains(FindText))
			{
				FoundIdx = i;
				break;
			}
		}

		if (FoundIdx != INDEX_NONE)
		{
			CurLogListView->SetSelection(CurTabLogLines[FoundIdx], ESelectInfo::OnKeyPress);
			CurLogListView->RequestScrollIntoView(CurTabLogLines[FoundIdx]);

			if (!InTab->bHighlightFind)
			{
				CurLogListView->RequestListRefresh();
			}
		}

		if (InTab->bHighlightFind)
		{
			UpdateFindHighlight(InTab, true, FindText);
		}


		// If the search wrapped-around or failed, displaying the error label briefly
		TSharedPtr<SBorder> ErrorLabel = InTab->FindErrorLabel;

		if (FoundIdx == INDEX_NONE || bSearchWrapped)
		{
			TSharedPtr<STextBlock> ErrorText = InTab->FindErrorText;

			if (FoundIdx == INDEX_NONE)
			{
				ErrorText->SetText(FText::FromString(TEXT("Failed to find text.")));
			}
			else if (bSearchUp)
			{
				ErrorText->SetText(FText::FromString(TEXT("Searched past top of list. Starting again from bottom.")));
			}
			else
			{
				ErrorText->SetText(FText::FromString(TEXT("Searched past bottom of list. Starting again from top.")));
			}

			ErrorLabel->SetBorderBackgroundColor(FLinearColor(1.f, 1.f, 1.f, 1.f));
			ErrorLabel->SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
			ErrorLabel->SetVisibility(EVisibility::HitTestInvisible);
		}
		// Otherwise, hide the label
		else
		{
			ErrorLabel->SetVisibility(EVisibility::Collapsed);
		}

		InTab->bLastFindWasUp = bSearchUp;
	}
}

bool SLogWidget::CanAutoScroll(TSharedPtr<FLogTabInfo> InTab)
{
	bool bReturnVal = true;

	if (InTab.IsValid())
	{
		// If the current tab is currently displaying the find bar, auto scrolling is not possible
		if (InTab->FindBox.IsValid() && InTab->FindBox->GetVisibility() != EVisibility::Collapsed)
		{
			bReturnVal = false;
		}
	}

	return bReturnVal;
}

void SLogWidget::UpdateFindHighlight(TSharedPtr<FLogTabInfo> InTab, bool bHighlight, FString HighlightText/*=TEXT("")*/)
{
	if (!bHighlight || HighlightText.Len() == 0)
	{
		for (TSharedRef<FLogLine>& CurLine : InTab->TabLogLines)
		{
			CurLine->LogHighlight.Empty();
		}
	}
	else
	{
		for (TSharedRef<FLogLine>& CurLine : InTab->TabLogLines)
		{
			if (CurLine->LogLine->Contains(HighlightText))
			{
				CurLine->LogHighlight = HighlightText;
			}
			else
			{
				CurLine->LogHighlight.Empty();
			}
		}
	}

	InTab->LogListView->RebuildList();
	InTab->LogListView->Invalidate(EInvalidateWidget::Layout);
}

void SLogWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	TSharedPtr<FLogTabInfo> CurActiveTab = GetActiveTabInfo();

	if (CurActiveTab.IsValid())
	{
		TSharedPtr<SBorder> ActiveFindLabel = CurActiveTab->FindErrorLabel;

		if (ActiveFindLabel.IsValid() && ActiveFindLabel->GetVisibility() != EVisibility::Collapsed)
		{
			const float FadeDuration = 2.f;
			FLinearColor CurColor = CALL_PROTECTED(SBorder, ActiveFindLabel, GetBorderBackgroundColor)().GetSpecifiedColor();

			CurColor.A -= (InDeltaTime / FadeDuration);

			if (CurColor.A > 0.1f)
			{
				ActiveFindLabel->SetBorderBackgroundColor(CurColor);
				ActiveFindLabel->SetColorAndOpacity(CurColor);
			}
			else
			{
				ActiveFindLabel->SetVisibility(EVisibility::Collapsed);
			}
		}
	}
}


