// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SLogWindow.h"
#include "UI/SLogWidget.h"

void SLogWindow::Construct(const FArguments& InArgs, FString InTitle, float WindowPosX, float WindowPosY, float WindowWidth,
								float WindowHeight)
{
	SWindow::Construct(SWindow::FArguments()
			.ClientSize(FVector2D(WindowWidth, WindowHeight))
			.ScreenPosition(FVector2D(WindowPosX, WindowPosY))
			.Title(FText::FromString(InTitle))
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::None)
			[
				SAssignNew(LogWidget, SLogWidget)
				.bStatusWidget(InArgs._bStatusWindow)
				.ExpectedFilters(InArgs._ExpectedFilters)
			]);

	// Due to ClientSize not accounting for the full size of the window (it's usually a bit oversized), fix that up now
	FVector2D OversizeSize = GetWindowSizeFromClientSize(FVector2D(WindowWidth, WindowHeight));

	Resize(FVector2D(WindowWidth - (OversizeSize.X - WindowWidth), WindowHeight - (OversizeSize.Y - WindowHeight)));

	SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SLogWindow::NotifyWindowClosed));
	SetOnWindowMoved(FOnWindowMoved::CreateSP(this, &SLogWindow::NotifyWindowMoved));
}

void SLogWindow::NotifyWindowClosed(const TSharedRef<SWindow>& ClosedWindow)
{
	MultiOnWindowClosed.Broadcast(ClosedWindow);
}

void SLogWindow::NotifyWindowMoved(const TSharedRef<SWindow>& MovedWindow)
{
	// Don't mark as moved, if the window is only being shown for the first time
	if (bHasEverBeenShown)
	{
		bHasMoved = true;
	}
}


