// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitLogging.h"
#include "UnitTestManager.h"
#include "UnitTest.h"


/**
 * Globals
 */

FUnitLogInterface*	GActiveLogInterface				= nullptr;
ELogType			GActiveLogTypeFlags				= ELogType::None;
FUnitLogInterface*	GActiveLogEngineEvent			= nullptr;
UWorld*				GActiveLogWorld					= nullptr;
UNetConnection*		GActiveReceiveUnitConnection	= nullptr;



/**
 * FUnitLogInterface
 */

UUnitTest* FUnitLogInterface::GetUnitTest()
{
	UUnitTest* ReturnVal = nullptr;

	for (UUnitTest* CurUnitTest : UUnitTestManager::Get()->ActiveUnitTests)
	{
		if (CurUnitTest == this)
		{
			ReturnVal = CurUnitTest;
			break;
		}
	}

	return ReturnVal;
}
