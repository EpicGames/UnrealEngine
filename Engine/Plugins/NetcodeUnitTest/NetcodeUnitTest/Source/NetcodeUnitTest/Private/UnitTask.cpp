// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTask.h"

#include "ClientUnitTest.h"


/**
 * EUnitTaskFlags
 */

int32 GetUnitTaskPriority(EUnitTaskFlags InFlags)
{
	int32 ReturnVal = 0;

	static_assert(EUnitTaskFlags::PriorityMask_Six == EUnitTaskFlags::PriorityMask_MAX, "Missing an EUnitTaskFlags entry.");

	if (!!(InFlags & EUnitTaskFlags::PriorityMask_Six))
	{
		ReturnVal = 6;
	}
	else if (!!(InFlags & EUnitTaskFlags::PriorityMask_Five))
	{
		ReturnVal = 5;
	}
	else if (!!(InFlags & EUnitTaskFlags::PriorityMask_Four))
	{
		ReturnVal = 4;
	}
	else if (!!(InFlags & EUnitTaskFlags::PriorityMask_Three))
	{
		ReturnVal = 3;
	}
	else if (!!(InFlags & EUnitTaskFlags::PriorityMask_Two))
	{
		ReturnVal = 2;
	}
	else if (InFlags == EUnitTaskFlags::PriorityMask_One)
	{
		ReturnVal = 1;
	}
	else
	{
		// There should be no flags without a priority level
		check(false);
	}

	return ReturnVal;
}


/**
 * UUnitTask
 */

UUnitTask::UUnitTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FUnitLogRedirect()
	, UnitTaskFlags(EUnitTaskFlags::None)
	, Owner(nullptr)
	, bStarted(false)
{
}

void UUnitTask::Attach(UUnitTest* InOwner)
{
	check(Owner == nullptr);

	Owner = InOwner;

	SetInterface(Owner);

	ValidateUnitTaskSettings();
}

void UUnitTask::ValidateUnitTaskSettings()
{
	// @todo #JohnB: Compile-time checks, later?

	// You can't block the unit test, if you require a server
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::BlockUnitTest) || !(UnitTaskFlags & EUnitTaskFlags::RequireServer));

	// You can't block the unit test, if you require a minimal client
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::BlockUnitTest) || !(UnitTaskFlags & EUnitTaskFlags::RequireMinClient));

	// You can't block the MinimalClient, if you're already blocking the entire unit test
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::BlockUnitTest) || !(UnitTaskFlags & EUnitTaskFlags::BlockMinClient));

	// Don't require a server, if the UnitTask is not attached to a ClientUnitTest
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::RequireServer) || Cast<UClientUnitTest>(Owner) != nullptr);

	// Don't require a MinimalClient, if the UnitTask is not attached to a ClientUnitTest
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::RequireMinClient) || Cast<UClientUnitTest>(Owner) != nullptr);

	// Don't block a MinimalClient, if the UnitTask is not attached to a ClientUnitTest
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::BlockMinClient) || Cast<UClientUnitTest>(Owner) != nullptr);

	// You can't require a MinimalClient while also blocking its creation
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::RequireMinClient) || !(UnitTaskFlags & EUnitTaskFlags::BlockMinClient));

	// You can't alter the MinimalClient, if you require it
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::AlterMinClient) || !(UnitTaskFlags & EUnitTaskFlags::RequireMinClient));

	// You can't alter the MinimalClient, if you block the unit test
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::AlterMinClient) || !(UnitTaskFlags & EUnitTaskFlags::BlockUnitTest));

	// You can't alter the MinimalClient, if you block it
	UNIT_ASSERT(!(UnitTaskFlags & EUnitTaskFlags::AlterMinClient) || !(UnitTaskFlags & EUnitTaskFlags::BlockMinClient));
}


void UUnitTask::TriggerFailure(FString Reason)
{
	check(Owner != nullptr);

	Owner->NotifyUnitTaskFailure(this, Reason);
} 

void UUnitTask::ResetTimeout(FString ResetReason, bool bResetConnTimeout/*=false*/, uint32 MinDuration/*=0*/)
{
	if (Owner != nullptr)
	{
		Owner->ResetTimeout(ResetReason, bResetConnTimeout, MinDuration);
	}
}
