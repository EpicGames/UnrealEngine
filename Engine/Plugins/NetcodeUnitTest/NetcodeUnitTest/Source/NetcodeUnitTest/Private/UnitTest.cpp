// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTest.h"

#include "HAL/FileManager.h"
#include "Containers/ArrayBuilder.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/FeedbackContext.h"

#include "Misc/Paths.h"
#include "UnitTestEnvironment.h"
#include "UnitTestManager.h"
#include "NUTUtil.h"

#include "UI/SLogWidget.h"
#include "UI/SLogWindow.h"


FUnitTestEnvironment* UUnitTest::UnitEnv = nullptr;
FUnitTestEnvironment* UUnitTest::NullUnitEnv = nullptr;


/**
 * EUnitTestResetStage
 */

FString GetUnitTestResetStageName(EUnitTestResetStage Stage)
{
	#define EUTRS_CASE(x)	case EUnitTestResetStage::x : return TEXT(#x)

	switch(Stage)
	{
		EUTRS_CASE(None);
		EUTRS_CASE(FullReset);
		EUTRS_CASE(ResetConnection);
		EUTRS_CASE(ResetExecute);

		default:
			return FString::Printf(TEXT("Unknown: %i"), (uint32)Stage);
	}

	#undef EUTRS_CASE
}


/**
 * UUnitTestBase
 */

UUnitTestBase::UUnitTestBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FUnitLogInterface()
{
}

bool UUnitTestBase::StartUnitTest()
{
	bool bReturnVal = false;

	UNIT_EVENT_BEGIN(this);

	bReturnVal = UTStartUnitTest();

	UNIT_EVENT_END;

	return bReturnVal;
}


/**
 * UUnitTest
 */

UUnitTest::UUnitTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, UnitTestName()
	, UnitTestType()
	, UnitTestDate(FDateTime::MinValue())
	, UnitTestBugTrackIDs()
	, UnitTestCLs()
	, bWorkInProgress(false)
	, bUnreliable(false)
	, bObsolete(false)
	, ExpectedResult()
	, UnitTestTimeout(0)
	, PeakMemoryUsage(0)
	, TimeToPeakMem(0.0f)
	, LastExecutionTime(0.0f)
	, LastNetTick(0.0)
	, CurrentMemoryUsage(0)
	, StartTime(0.0)
	, TimeoutExpire(0.0)
	, LastTimeoutReset(0.0)
	, LastTimeoutResetEvent()
	, bDeveloperMode(false)
	, bFirstTimeStats(false)
	, UnitTasks()
	, UnitTaskState(EUnitTaskFlags::None)
	, bCompleted(false)
	, VerificationState(EUnitTestVerification::Unverified)
	, bVerificationLogged(false)
	, bAborted(false)
	, LogWindow()
	, StatusLogSummary()
	, UnitLog()
	, UnitLogDir()
{
}

bool UUnitTest::ValidateUnitTestSettings(bool bCDOCheck/*=false*/)
{
	bool bSuccess = true;

	// The unit test must specify some ExpectedResult values
	UNIT_ASSERT(ExpectedResult.Num() > 0);


	TArray<EUnitTestVerification> ExpectedResultList;

	ExpectedResult.GenerateValueArray(ExpectedResultList);

	// Unit tests should not expect unreliable results, without being marked as unreliable
	UNIT_ASSERT(!ExpectedResultList.Contains(EUnitTestVerification::VerifiedUnreliable) || bUnreliable);

	// Unit tests should never expect 'needs-update' as a valid unit test result
	// @todo JohnB: It might make sense to allow this in the future, if you want to mark unit tests as 'needs-update' before running,
	//				so that you can skip running those unit tests
	UNIT_ASSERT(!ExpectedResultList.Contains(EUnitTestVerification::VerifiedNeedsUpdate));


	// Every unit test must specify a timeout value
	UNIT_ASSERT(UnitTestTimeout > 0);

	return bSuccess;
}

void UUnitTest::AddTask(UUnitTask* InTask)
{
	check(!HasStarted());

	int32 InsertIdx = 0;
	int32 CurPriority = GetUnitTaskPriority(InTask->GetUnitTaskFlags());

	// Determine the index at which the unit test should be added, based on execution priority (defaults to last)
	for (int32 CurIdx = UnitTasks.Num()-1; CurIdx >= 0; CurIdx--)
	{
		if (CurPriority >= GetUnitTaskPriority(UnitTasks[CurIdx]->GetUnitTaskFlags()))
		{
			InsertIdx = CurIdx + 1;
			break;
		}
	}

	UnitTasks.InsertUninitialized(InsertIdx);
	UnitTasks[InsertIdx] = InTask;

	InTask->Attach(this);
}

bool UUnitTest::UTStartUnitTest()
{
	bool bSuccess = false;

	InitializeLogs();

	StartTime = FPlatformTime::Seconds();

	if (bWorkInProgress)
	{
		UNIT_LOG(ELogType::StatusWarning, TEXT("WARNING: Unit test marked as 'work in progress', not included in automated tests."));
	}

	if (bObsolete)
	{
		UNIT_LOG(ELogType::StatusError, TEXT("Unit test marked as 'obsolete', and will most likely fail."))
	}

	if (UnitEnv != nullptr)
	{
		UnitTestTimeout = FMath::Max(UnitTestTimeout, UnitEnv->GetDefaultUnitTestTimeout());
	}

	bool bBlocked = IsTaskBlocking(EUnitTaskFlags::BlockUnitTest);

	if (bBlocked)
	{
		bSuccess = true;

		UnitTaskState |= EUnitTaskFlags::BlockUnitTest;
	}
	else
	{
		bSuccess = ExecuteUnitTest();

		if (bSuccess)
		{
			ResetTimeout(TEXT("StartUnitTest"));
		}
		else
		{
			CleanupUnitTest();
		}
	}

	return bSuccess;
}

void UUnitTest::AbortUnitTest()
{
	bAborted = true;

	UNIT_LOG(ELogType::StatusWarning, TEXT("Aborting unit test execution"));

	EndUnitTest();
}

void UUnitTest::EndUnitTest()
{
	if (!bAborted)
	{
		// Save the time it took to execute the unit test
		LastExecutionTime = (float)(FPlatformTime::Seconds() - StartTime);
		SaveConfig();

		UNIT_LOG(ELogType::StatusImportant, TEXT("Execution time: %f seconds"), LastExecutionTime);
	}

	// If we aborted, and this is the first run, wipe any stats that were written
	if (bAborted && IsFirstTimeStats())
	{
		PeakMemoryUsage = 0;

		SaveConfig();
	}


	CleanupUnitTest();
	bCompleted = true;

	if (GUnitTestManager != nullptr)
	{
		GUnitTestManager->NotifyUnitTestComplete(this, bAborted);
	}
}

void UUnitTest::CleanupUnitTest(EUnitTestResetStage ResetStage/*=EUniTestResetStage::None*/)
{
	if (ResetStage <= EUnitTestResetStage::FullReset)
	{
		for (UUnitTask* CurTask : UnitTasks)
		{
			CurTask->Cleanup();
		}

		UnitTasks.Empty();
	}


	// Only destroy/end the unit test, if it's being cleaned up fully, and not just reset
	if (ResetStage == EUnitTestResetStage::None)
	{
		if (GUnitTestManager != nullptr)
		{
			GUnitTestManager->NotifyUnitTestCleanup(this);
		}
		else
		{
			// Mark for garbage collection
			MarkAsGarbage();
		}
	}
	else
	{
		TimeoutExpire = 0.0;
		LastTimeoutReset = 0.0;
		LastTimeoutResetEvent = TEXT("");

		UnitTaskState = EUnitTaskFlags::None;

		bCompleted = false;
		VerificationState = EUnitTestVerification::Unverified;
		bVerificationLogged = false;
		bAborted = false;
	}
}

void UUnitTest::ResetUnitTest(EUnitTestResetStage ResetStage/*=EUniTestResetStage::FullReset*/)
{
	check(ResetStage != EUnitTestResetStage::None);

	if (CanResetUnitTest())
	{
		if (ResetStage > EUnitTestResetStage::FullReset)
		{
			UNIT_LOG(ELogType::StatusWarning, TEXT("Resetting unit test to stage '%s'."), *GetUnitTestResetStageName(ResetStage));
		}
		else
		{
			UNIT_LOG(ELogType::StatusWarning, TEXT("Resetting unit test."));
		}


		CleanupUnitTest(ResetStage);
	}
	else
	{
		UNIT_LOG(ELogType::StatusWarning, TEXT("Unit test does not support resetting."))
	}
}

void UUnitTest::InitializeLogs()
{
	if (UnitLogDir.IsEmpty())
	{
		UnitLogDir = UUnitTestManager::Get()->GetBaseUnitLogDir() + GetUnitTestName();

		if (FPaths::DirectoryExists(UnitLogDir))
		{
			uint32 UnitTestNameCount = 1;

			for (; FPaths::DirectoryExists(UnitLogDir + FString::Printf(TEXT("_%i"), UnitTestNameCount)); UnitTestNameCount++)
			{
				UNIT_ASSERT(UnitTestNameCount < 16384);
			}

			UnitLogDir += FString::Printf(TEXT("_%i"), UnitTestNameCount);
		}

		UnitLogDir += TEXT("/");

		IFileManager::Get().MakeDirectory(*UnitLogDir);

		UnitLog = MakeUnique<FOutputDeviceFile>(*(UnitLogDir + GetUnitTestName() + TEXT(".log")));
	}
}

void UUnitTest::NotifyLocalLog(ELogType InLogType, const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (LogWindow.IsValid())
	{
		TSharedPtr<SLogWidget>& LogWidget = LogWindow->LogWidget;

		ELogType LogOrigin = (InLogType & ELogType::OriginMask);

		if (LogOrigin != ELogType::None && !(LogOrigin & ELogType::OriginVoid))
		{
			if (UnitLog.IsValid() && Verbosity != ELogVerbosity::SetColor)
			{
				if (!!(LogOrigin & ELogType::OriginNet))
				{
					NUTUtil::SpecialLog(UnitLog.Get(), TEXT("[NET]"), Data, Verbosity, Category);
				}
				else
				{
					UnitLog->Serialize(Data, Verbosity, Category);
				}
			}

			if (LogWidget.IsValid())
			{
				if (Verbosity == ELogVerbosity::SetColor)
				{
					// Unimplemented
				}
				else
				{
					FString LogLine = FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Data, GPrintLogTimes);
					FSlateColor CurLogColor = FSlateColor::UseForeground();

					// Make unit test logs grey
					if (!!(LogOrigin & ELogType::OriginUnitTest))
					{
						if (LogColor.IsColorSpecified())
						{
							CurLogColor = LogColor;
						}
						else
						{
							CurLogColor = FLinearColor(0.25f, 0.25f, 0.25f);
						}
					}

					// Prefix net logs with [NET]
					if (!!(LogOrigin & ELogType::OriginNet))
					{
						LogLine = FString(TEXT("[NET]")) + LogLine;
					}

					// Override log color, if a warning or error is being printed
					if (Verbosity == ELogVerbosity::Error)
					{
						CurLogColor = FLinearColor(1.f, 0.f, 0.f);
					}
					else if (Verbosity == ELogVerbosity::Warning)
					{
						CurLogColor = FLinearColor(1.f, 1.f, 0.f);
					}

					bool bRequestFocus = !!(LogOrigin & ELogType::FocusMask);

					LogWidget->AddLine(InLogType, MakeShareable(new FString(LogLine)), CurLogColor, bRequestFocus);
				}
			}
		}
	}
}

void UUnitTest::NotifyStatusLog(ELogType InLogType, const TCHAR* Data)
{
	StatusLogSummary.Add(MakeShareable(new FUnitStatusLog(InLogType, FString(Data))));
}

bool UUnitTest::IsConnectionLogSource(UNetConnection* InConnection)
{
	bool bReturnVal = false;

	for (UUnitTask* CurTask : UnitTasks)
	{
		if (CurTask->IsConnectionLogSource(InConnection))
		{
			bReturnVal = true;
			break;
		}
	}

	return bReturnVal;
}

bool UUnitTest::IsTimerLogSource(UObject* InTimerDelegateObject)
{
	bool bReturnVal = false;

	for (UUnitTask* CurTask : UnitTasks)
	{
		if (CurTask->IsTimerLogSource(InTimerDelegateObject))
		{
			bReturnVal = true;
			break;
		}
	}

	return bReturnVal;
}

void UUnitTest::NotifyDeveloperModeRequest(bool bInDeveloperMode)
{
	bDeveloperMode = bInDeveloperMode;
}

bool UUnitTest::NotifyConsoleCommandRequest(FString CommandContext, FString Command)
{
	bool bHandled = false;
	static TArray<FString> BadCmds = TArrayBuilder<FString>()
		.Add("exit");

	// Don't execute commands that crash
	if (BadCmds.Contains(Command))
	{
		bHandled = true;

		UNIT_LOG(ELogType::OriginConsole,
					TEXT("Can't execute command '%s', it's in the 'bad commands' list (i.e. probably crashes)"), *Command);
	}

	if (!bHandled)
	{
		if (CommandContext == TEXT("Global"))
		{
			UNIT_LOG_BEGIN(this, ELogType::OriginConsole);
			bHandled = GEngine->Exec(nullptr, *Command, *GLog);
			UNIT_LOG_END();
		}
	}

	return bHandled;
}

void UUnitTest::GetCommandContextList(TArray<TSharedPtr<FString>>& OutList, FString& OutDefaultContext)
{
	OutList.Add(MakeShareable(new FString(TEXT("Global"))));

	OutDefaultContext = TEXT("Global");
}

void UUnitTest::UnitTick(float DeltaTime)
{
	if (UnitTasks.Num() > 0)
	{
		UUnitTask* CurTask = UnitTasks[0];

		// Detect tasks completed during the previous tick
		if (CurTask->HasStarted() && CurTask->IsTaskComplete())
		{
			FString Msg = FString::Printf(TEXT("UnitTask '%s' has completed."), *CurTask->GetName());

			UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *Msg);

			ResetTimeout(Msg);

			CurTask->Cleanup();
			UnitTasks.RemoveAt(0);

			CurTask = (UnitTasks.Num() > 0 ? UnitTasks[0] : nullptr);
		}


		// See if any blocked events are pending and ready to execute
		EUnitTaskFlags PendingEvents = UnitTaskState & EUnitTaskFlags::BlockMask;

		if (PendingEvents != EUnitTaskFlags::None)
		{
			// Limited only to PendingEvents that are blocked
			EUnitTaskFlags BlockedEvents = EUnitTaskFlags::None;

			for (UUnitTask* BlockTask : UnitTasks)
			{
				BlockedEvents |= (BlockTask->GetUnitTaskFlags() & PendingEvents);
			}

			EUnitTaskFlags ReadyEvents = PendingEvents ^ BlockedEvents;

			UnitTaskState &= ~ReadyEvents;

			UnblockEvents(ReadyEvents);
		}


		// If all of the current tasks requirements are met, begin execution
		if (CurTask != nullptr)
		{
			EUnitTaskFlags CurTaskRequirements = CurTask->GetUnitTaskFlags() & EUnitTaskFlags::RequirementsMask;
			EUnitTaskFlags MetRequirements = UnitTaskState & EUnitTaskFlags::RequirementsMask;

			if ((MetRequirements & CurTaskRequirements) == CurTaskRequirements)
			{
				if (!CurTask->HasStarted())
				{
					FString Msg = FString::Printf(TEXT("Starting UnitTask: '%s'."), *CurTask->GetName());

					UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *Msg);

					ResetTimeout(Msg);

					CurTask->bStarted = true;
					CurTask->StartTask();
				}

				if (CurTask->IsTickable())
				{
					CurTask->UnitTick(DeltaTime);
				}
			}
		}
	}
}

void UUnitTest::NetTick()
{
	if (UnitTasks.Num() > 0)
	{
		UUnitTask* CurTask = UnitTasks[0];

		if (CurTask->HasStarted() && CurTask->IsTickable())
		{
			CurTask->NetTick();
		}
	}
}

void UUnitTest::PostUnitTick(float DeltaTime)
{
	if (!bDeveloperMode && LastTimeoutReset != 0.0 && TimeoutExpire != 0.0)
	{
		double TimeSinceLastReset = FPlatformTime::Seconds() - LastTimeoutReset;

		if ((FPlatformTime::Seconds() - TimeoutExpire) > 0.0)
		{
			FString UpdateMsg = FString::Printf(TEXT("Unit test timed out after '%f' seconds - marking unit test as needing update."),
													TimeSinceLastReset);

			UNIT_LOG(ELogType::StatusFailure | ELogType::StyleBold, TEXT("%s"), *UpdateMsg);
			UNIT_STATUS_LOG(ELogType::StatusFailure | ELogType::StatusVerbose | ELogType::StyleBold, TEXT("%s"), *UpdateMsg);


			UpdateMsg = FString::Printf(TEXT("Last event to reset the timeout timer: '%s'"), *LastTimeoutResetEvent);

			UNIT_LOG(ELogType::StatusImportant, TEXT("%s"), *UpdateMsg);
			UNIT_STATUS_LOG(ELogType::StatusImportant | ELogType::StatusVerbose, TEXT("%s"), *UpdateMsg);


			VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
		}
	}
}

bool UUnitTest::IsTickable() const
{
	return UnitTasks.Num() > 0 || (LastTimeoutReset != 0.0 && TimeoutExpire != 0.0);
}

void UUnitTest::TickIsComplete(float DeltaTime)
{
	// If the unit test was verified as success/fail (i.e. as completed), end it now
	if (!bCompleted && VerificationState != EUnitTestVerification::Unverified)
	{
		if (!bVerificationLogged)
		{
			LogComplete();

			bVerificationLogged = true;
		}

		if (!bDeveloperMode)
		{
			EndUnitTest();
		}
	}
}

void UUnitTest::UnblockEvents(EUnitTaskFlags ReadyEvents)
{
	if (!!(ReadyEvents & EUnitTaskFlags::BlockUnitTest))
	{
		bool bSuccess = ExecuteUnitTest();

		if (bSuccess)
		{
			ResetTimeout(TEXT("StartUnitTest (Post-UnitTask-Block)"));
		}
		else
		{
			CleanupUnitTest();
		}
	}
}

void UUnitTest::NotifyUnitTaskFailure(UUnitTask* InTask, FString Reason)
{
	UNIT_LOG(ELogType::StatusFailure, TEXT("UnitTask '%s' failed execution - reason: %s"), *InTask->GetName(), *Reason);

	VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
}

void UUnitTest::LogComplete()
{
	if (VerificationState == EUnitTestVerification::VerifiedNotFixed)
	{
		UNIT_LOG(ELogType::StatusFailure, TEXT("Unit test verified as NOT FIXED"));
	}
	else if (VerificationState == EUnitTestVerification::VerifiedFixed)
	{
		UNIT_LOG(ELogType::StatusSuccess, TEXT("Unit test verified as FIXED"));
	}
	else if (VerificationState == EUnitTestVerification::VerifiedUnreliable)
	{
		UNIT_LOG(ELogType::StatusWarning, TEXT("Unit test verified as UNRELIABLE"));
	}
	else if (VerificationState == EUnitTestVerification::VerifiedNeedsUpdate)
	{
		UNIT_LOG(ELogType::StatusWarning | ELogType::StyleBold, TEXT("Unit test NEEDS UPDATE"));
	}
	else
	{
		// Don't forget to add new verification states
		UNIT_ASSERT(false);
	}
}

