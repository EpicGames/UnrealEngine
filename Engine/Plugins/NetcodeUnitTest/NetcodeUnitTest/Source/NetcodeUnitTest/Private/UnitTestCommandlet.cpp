// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTestCommandlet.h"

#include "Engine/GameInstance.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Slate/SceneViewport.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/SWindow.h"
#include "Engine/GameEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "NUTUtil.h"
#include "Net/NUTUtilNet.h"

#include "UnitTestManager.h"

#include "UI/SLogDialog.h"


// @todo #JohnBLowPri: Fix StandaloneRenderer support for static builds
#if !IS_MONOLITHIC
#include "StandaloneRenderer.h"
#endif




// @todo #JohnB_NUTClient: If you later end up doing test client instances that are unit tested against client netcode
//				(same way as you do server instances at the moment, for testing server netcode),
//				then this commandlet code is probably an excellent base for setting up minimal standalone clients like that,
//				as it's intended to strip-down running unit tests, from a minimal client itself


// Enable access to the private UGameEngine.GameInstance value, using the GET_PRIVATE macro
IMPLEMENT_GET_PRIVATE_VAR(UGameEngine, GameInstance, decltype(UGameEngine::GameInstance));

/** The exit-confirmation dialog, displayed to the user when all unit tests are complete */
static TSharedPtr<SWindow> ConfirmDialog=NULL;

/** Whether or not exiting the commandlet has been confirmed by user input yet */
static bool bConfirmedExit=false;


/**
 * UUnitTestCommandlet
 */
UUnitTestCommandlet::UUnitTestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Start off with GIsClient set to false, so we don't init viewport etc., but turn it on later for netcode
	IsClient = false;

	IsServer = false;
	IsEditor = false;

	LogToConsole = true;
	ShowErrorCount = true;
	ShowProgress = true;
}

int32 UUnitTestCommandlet::Main(const FString& Params)
{
	// @todo #JohnBLowPri: Fix StandaloneRenderer support for static builds (this is very hard to do,
	//						getting the game renderer going with commandlets, is very tricky)
#if IS_MONOLITHIC
	UE_LOGF(LogUnitTest, Log, "NetcodeUnitTest commandlet not currently supported in static/monolithic builds. "
		"You must disable the externs in SlateOpenGLExtensions.cpp to hack-unblock monolithic builds.");

#else


	// Load some core modules that LaunchEngineLoop.cpp no longer loads for commandlets
	if (FModuleManager::Get().ModuleExists(TEXT("XMPP")))
	{
		FModuleManager::Get().LoadModule(TEXT("XMPP"));
	}

	FModuleManager::Get().LoadModule(TEXT("HTTP"));
	FModuleManager::Get().LoadModule(TEXT("OnlineSubsystem"));
	FModuleManager::Get().LoadModule(TEXT("OnlineSubsystemUtils"));

	if (!IsEngineExitRequested())
	{
		GIsRunning = true;

		TSharedPtr<SViewport> SlateViewport;
		TSharedPtr<FSceneViewport> SlateViewportInterface;
		// Hack-set the engine GameViewport, so that setting GIsClient, doesn't cause an auto-exit
		// @todo JohnB: If you later remove the GIsClient setting code below, remove this as well
		if (GEngine->GameViewport == nullptr)
		{
			UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);

			if (GameEngine != nullptr)
			{
				// Create viewport ecosystem so that unit tests which test these don't encounter nullptrs. 
				// GameInstace = GameEngine->GameInstance;
				UGameInstance* GameInstance = GET_PRIVATE(UGameEngine, GameEngine, GameInstance);
				UGameViewportClient* NewViewport = NewObject<UGameViewportClient>(GameEngine);
				FWorldContext* CurContext = GameInstance->GetWorldContext();

				GameEngine->GameViewport = NewViewport;
				NewViewport->Init(*CurContext, GameInstance);

				SlateViewportInterface = FSceneViewport::Create(NewViewport, SAssignNew(SlateViewport, SViewport));

				// Set the main engine world context game viewport, to match the newly created viewport, in order to prevent crashes
				CurContext->GameViewport = NewViewport;
			}
		}


		// Now, after init stages are done, enable GIsClient for netcode and such
		GIsClient = true;

		// Required for the unit test commandlet
		PRIVATE_GAllowCommandletRendering = true;

		// Before running any unit tests, create a world object, which will contain the unit tests manager etc.
		//	(otherwise, when the last unit test world is cleaned up, the unit test manager stops functioning)
		UWorld* UnitTestWorld = NUTNet::CreateUnitTestWorld(false);


		const TCHAR* ParamsRef = *Params;
		FString UnitTestParam = TEXT("");
		FString UnitCmd = TEXT("UnitTest ");


		if (FParse::Value(ParamsRef, TEXT("UnitTest="), UnitTestParam))
		{
			UnitCmd += UnitTestParam;
		}
		else
		{
			UnitCmd += TEXT("all");
		}

		GEngine->Exec(UnitTestWorld, *UnitCmd);


		// NOTE: This main loop is partly based off of FileServerCommandlet
		while (GIsRunning && !IsEngineExitRequested())
		{
			CommandletHelpers::TickEngine(UnitTestWorld);

			FPlatformProcess::Sleep(0);


			// When the unit tests complete, open an exit-confirmation window, and when that closes, exit the main loop
			// NOTE: This will not execute, if the last open slate window is closed (such as when clicking 'yes' to 'abort all' dialog);
			//			in that circumstance, GIsRequestingExit gets set, by the internal engine code
			if (GUnitTestManager == nullptr || !GUnitTestManager->IsRunningUnitTests())
			{
				if (bConfirmedExit || FApp::IsUnattended())
				{
					// Wait until the status window is closed, if it is still open, before exiting
					if (GUnitTestManager == nullptr || !GUnitTestManager->StatusWindow.IsValid())
					{
						FPlatformMisc::RequestExit(false);
					}
				}
				else if (!ConfirmDialog.IsValid())
				{
					FText CompleteMsg = FText::FromString(TEXT("Unit test commandlet complete."));
					FText CompleteTitle = FText::FromString(TEXT("Unit tests complete"));

					FOnLogDialogResult DialogCallback = FOnLogDialogResult::CreateLambda(
						[&](const TSharedRef<SWindow>& DialogWindow, EAppReturnType::Type Result, bool bNoResult)
						{
							if (DialogWindow == ConfirmDialog)
							{
								bConfirmedExit = true;
							}
						});

					ConfirmDialog = OpenLogDialog_NonModal(EAppMsgType::Ok, CompleteMsg, CompleteTitle, DialogCallback);


					// If the 'abort all' dialog was open, close it now as it is redundant;
					// can't close it before opening above dialog though, otherwise no slate windows, triggers an early exit
					if (GUnitTestManager != nullptr && GUnitTestManager->AbortAllDialog.IsValid())
					{
						GUnitTestManager->AbortAllDialog->RequestDestroyWindow();

						GUnitTestManager->AbortAllDialog.Reset();
					}
				}
			}
		}

		// Cleanup the unit test world
		NUTNet::MarkUnitTestWorldForCleanup(UnitTestWorld, true);


		GIsRunning = false;
	}
#endif

	return (GWarn->GetNumErrors() == 0 ? 0 : 1);
}

void UUnitTestCommandlet::CreateCustomEngine(const FString& Params)
{
	// @todo #JohnBLowPri: Fix StandaloneRenderer support for static builds
#if !IS_MONOLITHIC
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());
#endif
}
