// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTestEnvironment.h"
#include "UnitTest.h"

#include "NUTUtil.h"


TMap<FString, FUnitTestEnvironment*> UnitTestEnvironments;


/**
 * FUnitTestEnvironment
 */

void FUnitTestEnvironment::AddUnitTestEnvironment(FString Game, FUnitTestEnvironment* Env)
{
	// @todo #JohnB: This memleaks Env if the game is already added (relevant to hot reload situations)
	// @todo #JohnB: Actually....this memleaks Env always! (including normal shutdown) Need to refactor this code - shared pointer?
	if (UnitTestEnvironments.Find(Game) == nullptr)
	{
		UnitTestEnvironments.Add(Game, Env);

		if (Game == FApp::GetProjectName())
		{
			UUnitTest::UnitEnv = Env;
		}
		else if (Game == TEXT("NullUnitEnv"))
		{
			UUnitTest::NullUnitEnv = Env;
		}
	}
}

FString FUnitTestEnvironment::GetDefaultMap(EUnitTestFlags UnitTestFlags)
{
	return TEXT("");
}

FString FUnitTestEnvironment::GetDefaultServerParameters(FString InLogCmds/*=TEXT("")*/, FString InExecCmds/*=TEXT("")*/)
{
	FString ReturnVal = TEXT("");
	FString CmdLineServerParms;
	FString FullLogCmds = InLogCmds;
	FString FullExecCmds = InExecCmds;

	if (NUTUtil::ParseValue(FCommandLine::Get(), TEXT("UnitTestServerParms="), CmdLineServerParms))
	{
		// @todo #JohnB: You need to add a log or notice, indicating that this invalidates the unit test as having a 'clean' run,
		//					without external influences affecting its environment (important, as this can cause unit tests to fail,
		//					when they depend on a log entry dependent on this commandline)

		auto ParseCmds =
			[&](const TCHAR* CmdsParm, FString& OutFullCmds)
			{
				FString Cmds;

				if (FParse::Value(*CmdLineServerParms, CmdsParm, Cmds, false))
				{
					if (OutFullCmds.Len() > 0)
					{
						OutFullCmds += TEXT(",");
					}

					OutFullCmds += Cmds.TrimQuotes();

					// Now remove the original *Cmds entry
					FString SearchStr = FString::Printf(TEXT("%s\"%s\""), CmdsParm, *Cmds.TrimQuotes());

					CmdLineServerParms = CmdLineServerParms.Replace(*SearchStr, TEXT(""), ESearchCase::IgnoreCase);
				}
			};


		// LogCmds and ExecCmds need to be specially merged, since they may be specified in muliple places
		// (e.g. within unit tests, and using UnitTestServerParms)
		ParseCmds(TEXT("-LogCmds="), FullLogCmds);
		ParseCmds(TEXT("-ExecCmds="), FullExecCmds);


		ReturnVal += TEXT(" ");
		ReturnVal += CmdLineServerParms;
	}

	SetupDefaultServerParameters(ReturnVal, FullLogCmds, FullExecCmds);

	if (FullLogCmds.Len() > 0)
	{
		ReturnVal += FString::Printf(TEXT(" -LogCmds=\"%s\""), *FullLogCmds);
	}

	if (FullExecCmds.Len() > 0)
	{
		ReturnVal += FString::Printf(TEXT(" -ExecCmds=\"%s\""), *FullExecCmds);
	}

	// Need to force all shader compilation, to happen with ShaderCompileWorker, so it can be detected easily;
	// sometimes it would occur within threads in the main UE process, which was not possible to detect, and which this disables
	ReturnVal += TEXT(" -ini:Engine:[DevOptions.Shaders]:bAllowAsynchronousShaderCompiling=False");

	// Need to disable Steam by default, on servers, to eliminate spurious warnings
	ReturnVal += TEXT(" -NoSteam");

	return ReturnVal;
}

FString FUnitTestEnvironment::GetDefaultClientParameters()
{
	FString ReturnVal = TEXT("");
	FString CmdLineClientParms;

	bool bDebugClient = FParse::Param(FCommandLine::Get(), TEXT("UnitTestClientDebug"));

	if (!bDebugClient)
	{
		ReturnVal = TEXT("-nullrhi -windowed -resx=640 -resy=480");
	}
	else
	{
		ReturnVal = TEXT("-windowed -resx=1024 -resy=768");
	}

	// @todo JohnB: Add merging of LogCmds, from above function, if adding default LogCmds for any game

	if (NUTUtil::ParseValue(FCommandLine::Get(), TEXT("UnitTestClientParms="), CmdLineClientParms))
	{
		ReturnVal += TEXT(" ");
		ReturnVal += CmdLineClientParms;
	}

	// Need to force all shader compilation, to happen with ShaderCompileWorker, so it can be detected easily;
	// sometimes it would occur within threads in the main UE process, which was not possible to detect, and which this disables
	ReturnVal += TEXT(" -ini:Engine:[DevOptions.Shaders]:bAllowAsynchronousShaderCompiling=False");

	SetupDefaultClientParameters(ReturnVal);

	return ReturnVal;
}

FString FUnitTestEnvironment::GetDefaultClientConnectURL()
{
	return TEXT("");
}

void FUnitTestEnvironment::GetServerProgressLogs(const TArray<FString>*& OutStartProgressLogs, const TArray<FString>*& OutReadyLogs,
													const TArray<FString>*& OutTimeoutResetLogs)
{
	static TArray<FString> StartProgressLogs;
	static TArray<FString> ReadyLogs;
	static TArray<FString> TimeoutResetLogs;
	static bool bSetupLogs = false;

	if (!bSetupLogs)
	{
		// Start progress logs common to all games
		StartProgressLogs.Add(TEXT("LogLoad: LoadMap: "));
		StartProgressLogs.Add(TEXT("LogUnitTest: NUTActor not present in RuntimeServerActors - adding this"));
		StartProgressLogs.Add(TEXT("LogNet: Spawning: /Script/NetcodeUnitTest.NUTActor"));

		// Logs which indicate that the server is ready
		ReadyLogs.Add(TEXT("LogWorld: Bringing up level for play took: "));

		// Logs which should trigger a timeout reset for all games
		TimeoutResetLogs.Add(TEXT("LogStaticMesh: Building static mesh "));
		TimeoutResetLogs.Add(TEXT("LogMaterial: Missing cached shader map for material "));
		TimeoutResetLogs.Add(TEXT("LogTexture:Display: Building textures: "));
		TimeoutResetLogs.Add(TEXT("Dumping tracked stack traces for TraceName '"));
		TimeoutResetLogs.Add(TEXT("Dumping once-off stack trace for TraceName '"));

		InitializeServerProgressLogs(StartProgressLogs, ReadyLogs, TimeoutResetLogs);

		bSetupLogs = true;
	}

	OutStartProgressLogs = &StartProgressLogs;
	OutReadyLogs = &ReadyLogs;
	OutTimeoutResetLogs = &TimeoutResetLogs;
}

void FUnitTestEnvironment::GetClientProgressLogs(const TArray<FString>*& OutTimeoutResetLogs)
{
	static TArray<FString> TimeoutResetLogs;
	static bool bSetupLogs = false;

	// @todo JohnB: See what logs are common between client/server, and perhaps create a third generic progress log function for them

	if (!bSetupLogs)
	{
		// Logs which should trigger a timeout reset for all games
		TimeoutResetLogs.Add(TEXT("LogStaticMesh: Building static mesh "));
		TimeoutResetLogs.Add(TEXT("LogMaterial: Missing cached shader map for material "));
		TimeoutResetLogs.Add(TEXT("LogTexture:Display: Building textures: "));

		InitializeClientProgressLogs(TimeoutResetLogs);

		bSetupLogs = true;
	}

	OutTimeoutResetLogs = &TimeoutResetLogs;
}

void FUnitTestEnvironment::GetProgressBlockingProcesses(const TArray<FString>*& OutBlockingProcesses)
{
	static TArray<FString> BlockingProcesses;
	static bool bSetupLogs = false;

	if (!bSetupLogs)
	{
		BlockingProcesses.Add(TEXT("ShaderCompileWorker"));

		InitializeProgressBlockingProcesses(BlockingProcesses);

		bSetupLogs = true;
	}

	OutBlockingProcesses = &BlockingProcesses;
}

