// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTests/Engine/PackedVectorTest.h"
#include "Engine/NetSerialization.h"
#include "NetcodeUnitTest.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include <limits>

namespace PackedVectorTest
{

	static bool AlmostEqualUlps(const float A, const float B, const int MaxUlps)
	{
		union FFloatToInt { float F; int32 I; };
		FFloatToInt IntA;
		FFloatToInt IntB;

		IntA.F = A;
		IntB.F = B;

		if ((IntA.I ^ IntB.I) < 0)
		{
			// For different signs we only allow +/- 0.0f
			return ((IntA.I | IntB.I) & 0x7FFFFFFF) == 0;
		}

		const int Ulps = FMath::Abs(IntA.I - IntB.I);
		return Ulps <= MaxUlps;
	}

}

UPackedVectorTest::UPackedVectorTest(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	UnitTestName = TEXT("PackedVector");
	UnitTestType = TEXT("Test");

	UnitTestDate = FDateTime(2018, 9, 19);

	ExpectedResult.Add(TEXT("NullUnitEnv"), EUnitTestVerification::VerifiedFixed);

	UnitTestTimeout = 5;
}

bool UPackedVectorTest::ExecuteUnitTest()
{
	TMap<FString, bool> TestResults;

	TestResults.Add(TEXT("Commencing Read/WritePackedVector tests. Only fails will be shown in log."), "Success" && true);

	const bool bFloatTestSuccess = ExecuteFloatTest(TestResults);
	const bool bDoubleTestSuccess = ExecuteDoubleTest(TestResults);
	const bool bWriteDoubleReadFloatTestSuccess = ExecuteWriteDoubleReadFloatTest(TestResults);

	VerifyTestResults(TestResults);

	return bFloatTestSuccess && bDoubleTestSuccess;
}

bool UPackedVectorTest::ExecuteFloatTest(TMap<FString, bool>& TestResults)
{
	static const float Quantize10_Values[] =
	{
		0.0f,
		-180817.42f,
		47.11f,
		-FMath::Exp2(25.0f),
		std::numeric_limits<float>::infinity(), // non-finite
	};

	static const float Quantize100_Values[] =
	{
		0.0f,
		+180720.42f,
		-19751216.0f,
		FMath::Exp2(31.0f),
		-std::numeric_limits<float>::infinity(),
	};

	struct TestCase
	{
		int ScaleFactor;

		const float* TestValues;
		int TestValueCount;

		TFunction<bool(FVector3f&, FArchive&)> Reader;
		TFunction<bool(FVector3f, FArchive&)> Writer;

	};

	TestCase TestCases[] =
	{
		{
			10,
			Quantize10_Values,
			sizeof(Quantize10_Values) / sizeof(Quantize10_Values[0]),
			[](FVector3f& Value, FArchive& Ar) { return ReadPackedVector<10, 24>(Value, Ar); },
			[](FVector3f Value, FArchive& Ar) { return WritePackedVector<10, 24>(Value, Ar); },
		},

		{
			100,
			Quantize100_Values,
			sizeof(Quantize100_Values) / sizeof(Quantize100_Values[0]),
			[](FVector3f& Value, FArchive& Ar) { return ReadPackedVector<100, 30>(Value, Ar); },
			[](FVector3f Value, FArchive& Ar) { return WritePackedVector<100, 30>(Value, Ar); },
		},
	};

	constexpr bool bAllowResize = false;
	FBitWriter Writer(128, bAllowResize);

	for (size_t TestCaseIt = 0, TestCaseEndIt = sizeof(TestCases)/sizeof(TestCases[0]); TestCaseIt != TestCaseEndIt; ++TestCaseIt)
	{
		const TestCase& Test = TestCases[TestCaseIt];
		for (size_t ValueIt = 0, ValueEndIt = Test.TestValueCount; ValueIt != ValueEndIt; ++ValueIt)
		{
			Writer.Reset();

			const float ScalarValue = Test.TestValues[ValueIt];
			const FVector3f WriteValue(ScalarValue);
			FVector3f ReadValue = FVector3f::ZeroVector;

			const bool bOverflowOrNan = !Test.Writer(WriteValue, Writer);
			bool LocalSuccess = !Writer.GetError();

			if (LocalSuccess)
			{
				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

				Test.Reader(ReadValue, Reader);
				LocalSuccess &= !Reader.GetError();
				LocalSuccess = (ReadValue.X == ReadValue.Y) && (ReadValue.X == ReadValue.Z);
				if (LocalSuccess)
				{
					// At this point we should have similar values as the original ones, except for NaN and overflowed values
					if (bOverflowOrNan)
					{
						if (WriteValue.ContainsNaN())
						{
							LocalSuccess &= (ReadValue == FVector3f::ZeroVector);
						}
						else
						{
							LocalSuccess = false;
						}
					}
					else
					{
						const float ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
						LocalSuccess &= ValueDiff < 2.0f/static_cast<float>(Test.ScaleFactor); // The diff test might need some adjustment
					}
				}
			}

			if (!LocalSuccess)
			{
				TestResults.Add(FString::Printf(TEXT("Read/WritePackedVector failed with scale %d and value %f. Got %f"), Test.ScaleFactor, ScalarValue, ReadValue.X), LocalSuccess);
			}
		}
	}

	return true;
}

bool UPackedVectorTest::ExecuteDoubleTest(TMap<FString, bool>& TestResults)
{
	static const double Quantize10_Values[] =
	{
		0.0f,
		-180817.42,
		47.11f,
		-FMath::Exp2(25.0),
		std::numeric_limits<double>::infinity(), // non-finite
	};

	static const double Quantize100_Values[] =
	{
		0.0f,
		+180720.42,
		-19751216.0,
		FMath::Exp2(40.0),
		FMath::Exp2(59.0),
		-std::numeric_limits<double>::infinity(),
	};

	struct TestCase
	{
		int ScaleFactor;

		const double* TestValues;
		int TestValueCount;

		TFunction<bool(FVector3d&, FArchive&)> Reader;
		TFunction<bool(FVector3d, FArchive&)> Writer;

	};

	TestCase TestCases[] =
	{
		{
			10,
			Quantize10_Values,
			sizeof(Quantize10_Values) / sizeof(Quantize10_Values[0]),
			[](FVector3d& Value, FArchive& Ar) { return ReadPackedVector<10, 24>(Value, Ar); },
			[](FVector3d Value, FArchive& Ar) { return WritePackedVector<10, 24>(Value, Ar); },
		},

		{
			100,
			Quantize100_Values,
			sizeof(Quantize100_Values) / sizeof(Quantize100_Values[0]),
			[](FVector3d& Value, FArchive& Ar) { return ReadPackedVector<100, 30>(Value, Ar); },
			[](FVector3d Value, FArchive& Ar) { return WritePackedVector<100, 30>(Value, Ar); },
		},
	};

	constexpr bool bAllowResize = false;
	FBitWriter Writer(512, bAllowResize);

	for (size_t TestCaseIt = 0, TestCaseEndIt = sizeof(TestCases)/sizeof(TestCases[0]); TestCaseIt != TestCaseEndIt; ++TestCaseIt)
	{
		const TestCase& Test = TestCases[TestCaseIt];
		for (size_t ValueIt = 0, ValueEndIt = Test.TestValueCount; ValueIt != ValueEndIt; ++ValueIt)
		{
			Writer.Reset();

			const double ScalarValue = Test.TestValues[ValueIt];
			const FVector3d WriteValue(ScalarValue);
			FVector3d ReadValue = FVector3d::ZeroVector;

			const bool bOverflowOrNan = !Test.Writer(WriteValue, Writer);
			bool LocalSuccess = !Writer.GetError();

			if (LocalSuccess)
			{
				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

				Test.Reader(ReadValue, Reader);
				LocalSuccess &= !Reader.GetError();
				LocalSuccess = (ReadValue.X == ReadValue.Y) && (ReadValue.X == ReadValue.Z);
				if (LocalSuccess)
				{
					// At this point we should have similar values as the original ones, except for NaN and overflowed values
					if (bOverflowOrNan)
					{
						if (WriteValue.ContainsNaN())
						{
							LocalSuccess &= (ReadValue == FVector3d::ZeroVector);
						}
						else
						{
							LocalSuccess = false;
						}
					}
					else
					{
						const double ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
						LocalSuccess &= ValueDiff < 2.0/Test.ScaleFactor; // The diff test might need some adjustment
					}
				}
			}

			if (!LocalSuccess)
			{
				TestResults.Add(FString::Printf(TEXT("Read/WritePackedVector failed with scale %d and value %f. Got %f"), Test.ScaleFactor, ScalarValue, ReadValue.X), LocalSuccess);
			}
		}
	}

	return true;
}

bool UPackedVectorTest::ExecuteWriteDoubleReadFloatTest(TMap<FString, bool>& TestResults)
{
	static const double Quantize100_Values[] =
	{
		0.0f,
		+180720.42,
		-19751216.0,
		FMath::Exp2(40.0),
		FMath::Exp2(59.0),
		-std::numeric_limits<double>::infinity(),
	};

	struct TestCase
	{
		int ScaleFactor;

		const double* TestValues;
		int TestValueCount;

		TFunction<bool(FVector3f&, FArchive&)> Reader;
		TFunction<bool(FVector3d, FArchive&)> Writer;

	};

	TestCase TestCases[] =
	{
		{
			100,
			Quantize100_Values,
			sizeof(Quantize100_Values) / sizeof(Quantize100_Values[0]),
			[](FVector3f& Value, FArchive& Ar) { return ReadPackedVector<100, 30>(Value, Ar); },
			[](FVector3d Value, FArchive& Ar) { return WritePackedVector<100, 30>(Value, Ar); },
		},
	};

	constexpr bool bAllowResize = false;
	FBitWriter Writer(512, bAllowResize);

	for (size_t TestCaseIt = 0, TestCaseEndIt = sizeof(TestCases)/sizeof(TestCases[0]); TestCaseIt != TestCaseEndIt; ++TestCaseIt)
	{
		const TestCase& Test = TestCases[TestCaseIt];
		for (size_t ValueIt = 0, ValueEndIt = Test.TestValueCount; ValueIt != ValueEndIt; ++ValueIt)
		{
			Writer.Reset();

			const double ScalarValue = Test.TestValues[ValueIt];
			const FVector3d WriteValue(ScalarValue);
			FVector3f ReadValue = FVector3f::ZeroVector;

			const bool bOverflowOrNan = !Test.Writer(WriteValue, Writer);
			bool LocalSuccess = !Writer.GetError();

			if (LocalSuccess)
			{
				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

				Test.Reader(ReadValue, Reader);
				LocalSuccess &= !Reader.GetError();
				LocalSuccess = (ReadValue.X == ReadValue.Y) && (ReadValue.X == ReadValue.Z);
				if (LocalSuccess)
				{
					// At this point we should have similar values as the original ones, except for NaN and overflowed values
					if (bOverflowOrNan)
					{
						if (WriteValue.ContainsNaN())
						{
							LocalSuccess &= (ReadValue == FVector3f::ZeroVector);
						}
						else
						{
							LocalSuccess = false;
						}
					}
					else
					{
						const double ValueDiff = FMath::Abs(ReadValue.X - WriteValue.X);
						LocalSuccess &= ValueDiff < 2.0/Test.ScaleFactor; // The diff test might need some adjustment
					}
				}
			}

			if (!LocalSuccess)
			{
				TestResults.Add(FString::Printf(TEXT("Read/WritePackedVector failed with scale %d and value %f. Got %f"), Test.ScaleFactor, ScalarValue, ReadValue.X), LocalSuccess);
			}
		}
	}

	return true;
}

void UPackedVectorTest::VerifyTestResults(const TMap<FString,bool>& TestResults)
{
	// Verify the results
	for (TMap<FString, bool>::TConstIterator It(TestResults); It; ++It)
	{
		UNIT_LOG(ELogType::StatusImportant, TEXT("Test '%s' returned: %s"), *It.Key(), (It.Value() ? TEXT("Success") : TEXT("FAIL")));

		if (!It.Value())
		{
			VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
		}
	}

	if (VerificationState == EUnitTestVerification::Unverified)
	{
		VerificationState = EUnitTestVerification::VerifiedFixed;
	}
}
