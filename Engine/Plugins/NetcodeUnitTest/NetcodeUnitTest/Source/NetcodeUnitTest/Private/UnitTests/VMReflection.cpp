// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTests/VMReflection.h"
#include "GameFramework/DefaultPawn.h"


#include "NUTUtilReflection.h"
#include "NetcodeUnitTest.h"


/**
 * UVMReflection
 */

UVMReflection::UVMReflection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UnitTestName = TEXT("VMReflection");
	UnitTestType = TEXT("Test");

	UnitTestDate = FDateTime(2015, 3, 23);

	ExpectedResult.Add(TEXT("NullUnitEnv"), EUnitTestVerification::VerifiedFixed);

	UnitTestTimeout = 60;
}

bool UVMReflection::ExecuteUnitTest()
{
	bool bSuccess = true;

	TMap<FString, bool> TestResults;


	/**
	 * Reflection functionality unit tests
	 */

	// Reflecting casting error reporting
	{
		UVMTestClassA* TestObjA = NewObject<UVMTestClassA>();
		UVMTestClassB* TestObjB = NewObject<UVMTestClassB>();

		TestObjA->AObjectRef = TestObjB;

		bool bOriginalError = false;
		(void)(UObject*)(FVMReflection(TestObjA)->*"AObjectRef", &bOriginalError);

		bool bError = false;
		(void)(FString)(FVMReflection(TestObjA)->*"AObjectRef", &bError);


		TestResults.Add(TEXT("Reflection casting error"), (!bOriginalError && bError));
	}


	/**
	 * Casting operator unit tests
	 */

	// UObject reflection and casting
	{
		UObject* TargetResult = AActor::StaticClass();
		UVMTestClassA* TestObjA = NewObject<UVMTestClassA>();
		UVMTestClassB* TestObjB = NewObject<UVMTestClassB>();

		TestObjA->AObjectRef = TestObjB;
		TestObjB->BObjectRef = TargetResult;

		bool bError = false;
		UObject* Result = (UObject*)(FVMReflection(TestObjA)->*"AObjectRef"->*"BObjectRef", &bError);

		TestResults.Add(TEXT("UObject Reflection"), (!bError && Result != NULL && Result == TargetResult));
	}

	// UObject property writing
	{
		UObject* TargetResult = AActor::StaticClass();
		UVMTestClassA* TestObjA = NewObject<UVMTestClassA>();
		UVMTestClassB* TestObjB = NewObject<UVMTestClassB>();

		TestObjA->AObjectRef = TestObjB;

		bool bError = false;
		UObject** Result = (UObject**)(FVMReflection(TestObjA)->*"AObjectRef", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("UObject Writing"), (!bError && TestObjA->AObjectRef == TargetResult));
	}

	// @todo #JohnB_VMRefl: Add a test for UObject** casting, i.e. writable object property references, now that it is supported



	// Byte property reading
	{
		uint8 TargetResult = 128;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->ByteProp = TargetResult;
		}

		bool bError = false;
		uint8 Result = (uint8)(FVMReflection(TestObj)->*"ByteProp", &bError);

		TestResults.Add(TEXT("Byte Reading"), (!bError && Result == TargetResult));
	}

	// Byte property writing
	{
		uint8 TargetResult = 64;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->ByteProp = 128;

		bool bError = false;
		uint8* Result = (uint8*)(FVMReflection(TestObj)->*"ByteProp", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("Byte Writing"), (!bError && TestObj->ByteProp == TargetResult));
	}



	// uint16 property reading
	{
		uint16 TargetResult = 512;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->UInt16Prop = TargetResult;
		}

		bool bError = false;
		uint16 Result = (uint16)(FVMReflection(TestObj)->*"UInt16Prop", &bError);

		TestResults.Add(TEXT("uint16 Reading"), (!bError && Result == TargetResult));
	}

	// uint16 property writing
	{
		uint16 TargetResult = 1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->UInt16Prop = 128;

		bool bError = false;
		uint16* Result = (uint16*)(FVMReflection(TestObj)->*"UInt16Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("uint16 Writing"), (!bError && TestObj->UInt16Prop == TargetResult));
	}

	// uint16 Byte property upcast reading
	{
		uint16 TargetResult = 128;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->ByteProp = (uint8)TargetResult;
		}

		bool bError = false;
		uint16 Result = (uint16)(FVMReflection(TestObj)->*"ByteProp", &bError);

		TestResults.Add(TEXT("uint16 Byte upcast Reading"), (!bError && Result == TargetResult));
	}



	// uint32 property reading
	{
		uint32 TargetResult = 131070;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->UInt32Prop = TargetResult;
		}

		bool bError = false;
		uint32 Result = (uint32)(FVMReflection(TestObj)->*"UInt32Prop", &bError);

		TestResults.Add(TEXT("uint32 Reading"), (!bError && Result == TargetResult));
	}

	// uint32 property writing
	{
		uint32 TargetResult = 262140;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->UInt32Prop = 128;

		bool bError = false;
		uint32* Result = (uint32*)(FVMReflection(TestObj)->*"UInt32Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("uint32 Writing"), (!bError && TestObj->UInt32Prop == TargetResult));
	}

	// uint32 Byte property upcast reading
	{
		uint32 TargetResult = 128;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->ByteProp = (uint8)TargetResult;
		}

		bool bError = false;
		uint32 Result = (uint32)(FVMReflection(TestObj)->*"ByteProp", &bError);

		TestResults.Add(TEXT("uint32 Byte upcast Reading"), (!bError && Result == TargetResult));
	}

	// uint32 uint16 property upcast reading
	{
		uint32 TargetResult = 1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->UInt16Prop = (uint16)TargetResult;
		}

		bool bError = false;
		uint32 Result = (uint32)(FVMReflection(TestObj)->*"UInt16Prop", &bError);

		TestResults.Add(TEXT("uint32 uint16 upcast Reading"), (!bError && Result == TargetResult));
	}


	// uint64 property reading
	{
		uint64 TargetResult = 8589934591ull;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->UInt64Prop = TargetResult;
		}

		bool bError = false;
		uint64 Result = (uint64)(FVMReflection(TestObj)->*"UInt64Prop", &bError);

		TestResults.Add(TEXT("uint64 Reading"), (!bError && Result == TargetResult));
	}

	// uint64 property writing
	{
		uint64 TargetResult = 17179869182ull;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->UInt64Prop = 128;

		bool bError = false;
		uint64* Result = (uint64*)(FVMReflection(TestObj)->*"UInt64Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("uint64 Writing"), (!bError && TestObj->UInt64Prop == TargetResult));
	}

	// uint64 Byte property upcast reading
	{
		uint64 TargetResult = 128;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->ByteProp = (uint8)TargetResult;
		}

		bool bError = false;
		uint64 Result = (uint64)(FVMReflection(TestObj)->*"ByteProp", &bError);

		TestResults.Add(TEXT("uint64 Byte upcast Reading"), (!bError && Result == TargetResult));
	}

	// uint64 uint16 property upcast reading
	{
		uint64 TargetResult = 1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->UInt16Prop = (uint16)TargetResult;
		}

		bool bError = false;
		uint64 Result = (uint64)(FVMReflection(TestObj)->*"UInt16Prop", &bError);

		TestResults.Add(TEXT("uint64 uint16 upcast Reading"), (!bError && Result == TargetResult));
	}

	// uint64 uint32 property upcast reading
	{
		uint64 TargetResult = 2147483647;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->UInt32Prop = (uint32)TargetResult;
		}

		bool bError = false;
		uint64 Result = (uint64)(FVMReflection(TestObj)->*"UInt32Prop", &bError);

		TestResults.Add(TEXT("uint64 uint32 upcast Reading"), (!bError && Result == TargetResult));
	}



	// int8 property reading
	{
		int8 TargetResult = -128;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int8Prop = TargetResult;
		}

		bool bError = false;
		int8 Result = (int8)(FVMReflection(TestObj)->*"Int8Prop", &bError);

		TestResults.Add(TEXT("int8 Reading"), (!bError && Result == TargetResult));
	}

	// int8 property writing
	{
		int8 TargetResult = -64;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->Int8Prop = 127;

		bool bError = false;
		int8* Result = (int8*)(FVMReflection(TestObj)->*"Int8Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("int8 Writing"), (!bError && TestObj->Int8Prop == TargetResult));
	}



	// int16 property reading
	{
		int16 TargetResult = -512;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int16Prop = TargetResult;
		}

		bool bError = false;
		int16 Result = (int16)(FVMReflection(TestObj)->*"Int16Prop", &bError);

		TestResults.Add(TEXT("int16 Reading"), (!bError && Result == TargetResult));
	}

	// int16 property writing
	{
		int16 TargetResult = -1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->Int16Prop = 128;

		bool bError = false;
		int16* Result = (int16*)(FVMReflection(TestObj)->*"Int16Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("int16 Writing"), (!bError && TestObj->Int16Prop == TargetResult));
	}

	// int16 int8 property upcast reading
	{
		int16 TargetResult = -64;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int8Prop = (int8)TargetResult;
		}

		bool bError = false;
		int16 Result = (int16)(FVMReflection(TestObj)->*"Int8Prop", &bError);

		TestResults.Add(TEXT("int16 int8 upcast Reading"), (!bError && Result == TargetResult));
	}



	// int32 property reading
	{
		int32 TargetResult = -131070;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int32Prop = TargetResult;
		}

		bool bError = false;
		int32 Result = (int32)(FVMReflection(TestObj)->*"Int32Prop", &bError);

		TestResults.Add(TEXT("int32 Reading"), (!bError && Result == TargetResult));
	}

	// int32 property writing
	{
		int32 TargetResult = -131070;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->Int32Prop = 128;

		bool bError = false;
		int32* Result = (int32*)(FVMReflection(TestObj)->*"Int32Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("int32 Writing"), (!bError && TestObj->Int32Prop == TargetResult));
	}

	// int32 int8 property upcast reading
	{
		int32 TargetResult = -64;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int8Prop = (int8)TargetResult;
		}

		bool bError = false;
		int32 Result = (int32)(FVMReflection(TestObj)->*"Int8Prop", &bError);

		TestResults.Add(TEXT("int32 int8 upcast Reading"), (!bError && Result == TargetResult));
	}

	// int32 int16 property upcast reading
	{
		int32 TargetResult = -1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int16Prop = (int16)TargetResult;
		}

		bool bError = false;
		int32 Result = (int32)(FVMReflection(TestObj)->*"Int16Prop", &bError);

		TestResults.Add(TEXT("int32 int16 upcast Reading"), (!bError && Result == TargetResult));
	}



	// int64 property reading
	{
		int64 TargetResult = -8589934591ll;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int64Prop = TargetResult;
		}

		bool bError = false;
		int64 Result = (int64)(FVMReflection(TestObj)->*"Int64Prop", &bError);

		TestResults.Add(TEXT("int64 Reading"), (!bError && Result == TargetResult));
	}

	// int64 property writing
	{
		int64 TargetResult = -8589934591ll;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->Int64Prop = 128;

		bool bError = false;
		int64* Result = (int64*)(FVMReflection(TestObj)->*"Int64Prop", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("int64 Writing"), (!bError && TestObj->Int64Prop == TargetResult));
	}

	// int64 int8 property upcast reading
	{
		int64 TargetResult = -64;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int8Prop = (int8)TargetResult;
		}

		bool bError = false;
		int64 Result = (int64)(FVMReflection(TestObj)->*"Int8Prop", &bError);

		TestResults.Add(TEXT("int64 int8 upcast Reading"), (!bError && Result == TargetResult));
	}

	// int64 int16 property upcast reading
	{
		int64 TargetResult = -1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int16Prop = (int16)TargetResult;
		}

		bool bError = false;
		int64 Result = (int64)(FVMReflection(TestObj)->*"Int16Prop", &bError);

		TestResults.Add(TEXT("int64 int16 upcast Reading"), (!bError && Result == TargetResult));
	}

	// int64 int32 property upcast reading
	{
		int64 TargetResult = -1073741823;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->Int32Prop = (int32)TargetResult;
		}

		bool bError = false;
		int64 Result = (int64)(FVMReflection(TestObj)->*"Int32Prop", &bError);

		TestResults.Add(TEXT("int64 int32 upcast Reading"), (!bError && Result == TargetResult));
	}



	// Float property reading
	{
		float TargetResult = 12.8f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->FloatProp = TargetResult;
		}

		bool bError = false;
		float Result = (float)(FVMReflection(TestObj)->*"FloatProp", &bError);

		TestResults.Add(TEXT("Float Reading"), (!bError && Result == TargetResult));
	}

	// Float property writing
	{
		float TargetResult = 6.4f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->FloatProp = 12.8f;

		bool bError = false;
		float* Result = (float*)(FVMReflection(TestObj)->*"FloatProp", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("Float Writing"), (!bError && TestObj->FloatProp == TargetResult));
	}



	// Double property reading
	{
		double TargetResult = 12.8;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DoubleProp = TargetResult;
		}

		bool bError = false;
		double Result = (double)(FVMReflection(TestObj)->*"DoubleProp", &bError);

		TestResults.Add(TEXT("Double Reading"), (!bError && Result == TargetResult));
	}

	// Double property writing
	{
		double TargetResult = 6.4;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->DoubleProp = 12.8;

		bool bError = false;
		double* Result = (double*)(FVMReflection(TestObj)->*"DoubleProp", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("Double Writing"), (!bError && TestObj->DoubleProp == TargetResult));
	}

	// Double Float property upcast reading
	{
		double TargetResult = (double)12.8f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->FloatProp = (float)TargetResult;
		}

		bool bError = false;
		double Result = (double)(FVMReflection(TestObj)->*"FloatProp", &bError);

		TestResults.Add(TEXT("Double Float upcast Reading"), (!bError && Result == TargetResult));
	}



	// Bool property reading
	{
		bool TargetResults[5] = {false, true, false, false, true};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->bBoolPropA = TargetResults[0];
			TestObj->bBoolPropB = TargetResults[1];
			TestObj->bBoolPropC = TargetResults[2];
			TestObj->bBoolPropD = TargetResults[3];
			TestObj->bBoolPropE = TargetResults[4];
		}

		bool bError = false;
		bool Results[5];

		Results[0] = (bool)(FVMReflection(TestObj)->*"bBoolPropA", &bError);
		Results[1] = !(bError || !((bool)(FVMReflection(TestObj)->*"bBoolPropB", &bError)));
		Results[2] = !(bError || !((bool)(FVMReflection(TestObj)->*"bBoolPropC", &bError)));
		Results[3] = !(bError || !((bool)(FVMReflection(TestObj)->*"bBoolPropD", &bError)));
		Results[4] = !(bError || !((bool)(FVMReflection(TestObj)->*"bBoolPropE", &bError)));

		TestResults.Add(TEXT("Bool reading"), (!bError && Results[0] == TargetResults[0] && Results[1] == TargetResults[1] &&
												Results[2] == TargetResults[2] && Results[3] == TargetResults[3] &&
												Results[4] == TargetResults[4]));
	}


	// Name property reading
	{
		FName TargetResult = NAME_BeaconNetDriver;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->NameProp = TargetResult;
		}

		bool bError = false;
		FName Result = (FName)(FVMReflection(TestObj)->*"NameProp", &bError);

		TestResults.Add(TEXT("Name Reading"), (!bError && Result == TargetResult));
	}

	// Name property writing
	{
		FName TargetResult = NAME_Camera;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->NameProp = NAME_Cylinder;

		bool bError = false;
		FName* Result = (FName*)(FVMReflection(TestObj)->*"NameProp", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("Name Writing"), (!bError && TestObj->NameProp == TargetResult));
	}



	// FString property reading
	{
		FString TargetResult = TEXT("TargetResult");
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->StringProp = TargetResult;
		}

		bool bError = false;
		FString Result = (FString)(FVMReflection(TestObj)->*"StringProp", &bError);

		TestResults.Add(TEXT("FString Reading"), (!bError && Result == TargetResult));
	}

	// FString property writing
	{
		FString TargetResult = TEXT("Expected");
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->StringProp = TEXT("NotExpected");

		bool bError = false;
		FString* Result = (FString*)(FVMReflection(TestObj)->*"StringProp", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("FString Writing"), (!bError && TestObj->StringProp == TargetResult));
	}



	// FText property reading
	{
		FText TargetResult = FText::FromString(TEXT("TargetResult"));
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->TextProp = TargetResult;
		}

		bool bError = false;
		FText Result = (FText)(FVMReflection(TestObj)->*"TextProp", &bError);

		TestResults.Add(TEXT("FText Reading"), (!bError && Result.EqualTo(TargetResult)));
	}

	// FText property writing
	{
		FText TargetResult = FText::FromString(TEXT("Expected"));
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->TextProp = FText::FromString(TEXT("NotExpected"));

		bool bError = false;
		FText* Result = (FText*)(FVMReflection(TestObj)->*"TextProp", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("FText Writing"), (!bError && TestObj->TextProp.EqualTo(TargetResult)));
	}



	/**
	 * Array unit tests
	 */

	// Bad static array access (no element selected)
	{
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();
		bool bError = false;

		(void)(uint8)((FVMReflection(TestObj)->*"BytePropArray")["uint8"], &bError);

		TestResults.Add(TEXT("Bad static array access (no element)"), bError);
	}

	// Bad static array access (no type verification specified)
	{
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();
		bool bError = false;

		(void)(uint8)(FVMReflection(TestObj)->*"BytePropArray", &bError);

		TestResults.Add(TEXT("Bad static array access (no type verification)"), bError);
	}


	// Static array reading
	{
		uint8 TargetResults[4] = {128, 64, 32, 16};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->BytePropArray[0] = TargetResults[0];
			TestObj->BytePropArray[1] = TargetResults[1];
			TestObj->BytePropArray[2] = TargetResults[2];
			TestObj->BytePropArray[3] = TargetResults[3];
		}

		bool bError[4] = {false, false, false, false};
		uint8 Results[4];
		
		Results[0] = (uint8)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][0], &bError[0]);
		Results[1] = (uint8)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][1], &bError[1]);
		Results[2] = (uint8)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][2], &bError[2]);
		Results[3] = (uint8)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][3], &bError[3]);

		TestResults.Add(TEXT("Static Array Reading"), (!bError[0] && Results[0] == TargetResults[0]) &&
						(!bError[1] && Results[1] == TargetResults[1]) && (!bError[2] && Results[2] == TargetResults[2]) &&
						(!bError[3] && Results[3] == TargetResults[3]));
	}

	// Static array writing
	{
		uint8 TargetResults[4] = {31, 15, 24, 47};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->BytePropArray[0] = 128;
		TestObj->BytePropArray[1] = 128;
		TestObj->BytePropArray[2] = 128;
		TestObj->BytePropArray[3] = 128;

		bool bError[4] = {false, false, false, false};
		uint8* Results[4];
		
		Results[0] = (uint8*)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][0], &bError[0]);
		Results[1] = (uint8*)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][1], &bError[1]);
		Results[2] = (uint8*)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][2], &bError[2]);
		Results[3] = (uint8*)((FVMReflection(TestObj)->*"BytePropArray")["uint8"][3], &bError[3]);

		for (int32 i=0; i<UE_ARRAY_COUNT(Results); i++)
		{
			if (!bError[i] && Results[i] != NULL)
			{
				*Results[i] = TargetResults[i];
			}
		}

		TestResults.Add(TEXT("Static Array Writing"), (!bError[0] && TestObj->BytePropArray[0] == TargetResults[0]) &&
							(!bError[1] && TestObj->BytePropArray[1] == TargetResults[1]) &&
							(!bError[2] && TestObj->BytePropArray[2] == TargetResults[2]) &&
							(!bError[3] && TestObj->BytePropArray[3] == TargetResults[3]));
	}


	// Static object array reflection
	{
		UObject* TargetResults[2] = {UObject::StaticClass(), AActor::StaticClass()};
		UVMTestClassA* TestObjA = NewObject<UVMTestClassA>();
		UVMTestClassB* TestObjB[2] = {NewObject<UVMTestClassB>(), NewObject<UVMTestClassB>()};

		if (TestObjA != NULL)
		{
			TestObjA->ObjectPropArray[0] = TestObjB[0];
			TestObjA->ObjectPropArray[1] = TestObjB[1];

			TestObjB[0]->BObjectRef = TargetResults[0];
			TestObjB[1]->BObjectRef = TargetResults[1];
		}

		bool bError[2] = {false, false};
		UObject* Results[2];
		
		Results[0] = (UObject*)((FVMReflection(TestObjA)->*"ObjectPropArray")["UObject*"][0]->*"BObjectRef", &bError[0]);
		Results[1] = (UObject*)((FVMReflection(TestObjA)->*"ObjectPropArray")["UObject*"][1]->*"BObjectRef", &bError[1]);

		TestResults.Add(TEXT("Static Object Array Reflection"), (!bError[0] && Results[0] == TargetResults[0]) &&
						(!bError[1] && Results[1] == TargetResults[1]));
	}



	// Bad dynamic array access (no element selected)
	{
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();
		bool bError = false;

		(void)(uint8)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"], &bError);

		TestResults.Add(TEXT("Bad dynamic array access (no element)"), bError);
	}

	// Bad dynamic array access (no type verification)
	{
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();
		bool bError = false;

		(void)(uint8)(FVMReflection(TestObj)->*"DynBytePropArray", &bError);

		TestResults.Add(TEXT("Bad dynamic array access (no type verification)"), bError);
	}


	// Dynamic array reading
	{
		uint8 TargetResults[4] = {128, 64, 32, 16};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynBytePropArray.Empty();
			TestObj->DynBytePropArray.AddZeroed(4);

			TestObj->DynBytePropArray[0] = TargetResults[0];
			TestObj->DynBytePropArray[1] = TargetResults[1];
			TestObj->DynBytePropArray[2] = TargetResults[2];
			TestObj->DynBytePropArray[3] = TargetResults[3];
		}

		bool bError[4] = {false, false, false, false};
		uint8 Results[4];
		
		Results[0] = (uint8)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][0], &bError[0]);
		Results[1] = (uint8)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][1], &bError[1]);
		Results[2] = (uint8)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][2], &bError[2]);
		Results[3] = (uint8)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][3], &bError[3]);

		TestResults.Add(TEXT("Dynamic Array Reading"), (!bError[0] && Results[0] == TargetResults[0]) &&
						(!bError[1] && Results[1] == TargetResults[1]) && (!bError[2] && Results[2] == TargetResults[2]) &&
						(!bError[3] && Results[3] == TargetResults[3]));
	}

	// Dynamic array writing
	{
		uint8 TargetResults[4] = {31, 15, 24, 47};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();


		TestObj->DynBytePropArray.Empty();
		TestObj->DynBytePropArray.AddZeroed(4);

		TestObj->DynBytePropArray[0] = 128;
		TestObj->DynBytePropArray[1] = 128;
		TestObj->DynBytePropArray[2] = 128;
		TestObj->DynBytePropArray[3] = 128;

		bool bError[4] = {false, false, false, false};
		uint8* Results[4];
		
		Results[0] = (uint8*)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][0], &bError[0]);
		Results[1] = (uint8*)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][1], &bError[1]);
		Results[2] = (uint8*)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][2], &bError[2]);
		Results[3] = (uint8*)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][3], &bError[3]);

		for (int32 i=0; i<UE_ARRAY_COUNT(Results); i++)
		{
			if (!bError[i] && Results[i] != NULL)
			{
				*Results[i] = TargetResults[i];
			}
		}

		TestResults.Add(TEXT("Dynamic Array Writing"), (!bError[0] && TestObj->DynBytePropArray[0] == TargetResults[0]) &&
							(!bError[1] && TestObj->DynBytePropArray[1] == TargetResults[1]) &&
							(!bError[2] && TestObj->DynBytePropArray[2] == TargetResults[2]) &&
							(!bError[3] && TestObj->DynBytePropArray[3] == TargetResults[3]));
	}

	// Dynamic bool array reading
	{
		bool TargetResults[4] = {true, false, false, true};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynBoolPropArray.Empty();
			TestObj->DynBoolPropArray.AddZeroed(4);

			TestObj->DynBoolPropArray[0] = TargetResults[0];
			TestObj->DynBoolPropArray[1] = TargetResults[1];
			TestObj->DynBoolPropArray[2] = TargetResults[2];
			TestObj->DynBoolPropArray[3] = TargetResults[3];
		}

		bool bError[4] = {false, false, false, false};
		bool Results[4];
		
		Results[0] = (bool)((FVMReflection(TestObj)->*"DynBoolPropArray")["bool"][0], &bError[0]);
		Results[1] = (bool)((FVMReflection(TestObj)->*"DynBoolPropArray")["bool"][1], &bError[1]);
		Results[2] = (bool)((FVMReflection(TestObj)->*"DynBoolPropArray")["bool"][2], &bError[2]);
		Results[3] = (bool)((FVMReflection(TestObj)->*"DynBoolPropArray")["bool"][3], &bError[3]);

		TestResults.Add(TEXT("Dynamic Bool Array Reading"), (!bError[0] && Results[0] == TargetResults[0]) &&
						(!bError[1] && Results[1] == TargetResults[1]) && (!bError[2] && Results[2] == TargetResults[2]) &&
						(!bError[3] && Results[3] == TargetResults[3]));
	}

	// Dynamic object array reflection
	{
		UObject* TargetResults[2] = {UObject::StaticClass(), AActor::StaticClass()};
		UVMTestClassA* TestObjA = NewObject<UVMTestClassA>();
		UVMTestClassB* TestObjB[2] = {NewObject<UVMTestClassB>(), NewObject<UVMTestClassB>()};

		if (TestObjA != NULL)
		{
			TestObjA->DynObjectPropArray.Empty();
			TestObjA->DynObjectPropArray.AddZeroed(2);

			TestObjA->DynObjectPropArray[0] = TestObjB[0];
			TestObjA->DynObjectPropArray[1] = TestObjB[1];

			TestObjB[0]->BObjectRef = TargetResults[0];
			TestObjB[1]->BObjectRef = TargetResults[1];
		}

		bool bError[2] = {false, false};
		UObject* Results[2];
		
		Results[0] = (UObject*)((FVMReflection(TestObjA)->*"DynObjectPropArray")["UObject*"][0]->*"BObjectRef", &bError[0]);
		Results[1] = (UObject*)((FVMReflection(TestObjA)->*"DynObjectPropArray")["UObject*"][1]->*"BObjectRef", &bError[1]);

		TestResults.Add(TEXT("Dynamic Object Array Reflection"), (!bError[0] && Results[0] == TargetResults[0]) &&
						(!bError[1] && Results[1] == TargetResults[1]));
	}


	// Dynamic array type verification (bool)
	{
		bool TargetResult = true;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynBoolPropArray.Empty();
			TestObj->DynBoolPropArray.AddZeroed(1);

			TestObj->DynBoolPropArray[0] = TargetResult;
		}

		bool bError = false;
		bool Result;
		
		Result = (bool)((FVMReflection(TestObj)->*"DynBoolPropArray")["bool"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (bool)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (FName)
	{
		FName TargetResult = NAME_Camera;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynNamePropArray.Empty();
			TestObj->DynNamePropArray.AddZeroed(1);

			TestObj->DynNamePropArray[0] = TargetResult;
		}

		bool bError = false;
		FName Result;
		
		Result = (FName)((FVMReflection(TestObj)->*"DynNamePropArray")["FName"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (FName)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (byte)
	{
		uint8 TargetResult = 92;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynBytePropArray.Empty();
			TestObj->DynBytePropArray.AddZeroed(1);

			TestObj->DynBytePropArray[0] = TargetResult;
		}

		bool bError = false;
		uint8 Result;
		
		Result = (uint8)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (byte)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (double)
	{
		double TargetResult = 9.2;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynDoublePropArray.Empty();
			TestObj->DynDoublePropArray.AddZeroed(1);

			TestObj->DynDoublePropArray[0] = TargetResult;
		}

		bool bError = false;
		double Result;
		
		Result = (double)((FVMReflection(TestObj)->*"DynDoublePropArray")["double"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (double)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (float)
	{
		float TargetResult = 8.4f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynFloatPropArray.Empty();
			TestObj->DynFloatPropArray.AddZeroed(1);

			TestObj->DynFloatPropArray[0] = TargetResult;
		}

		bool bError = false;
		float Result;
		
		Result = (float)((FVMReflection(TestObj)->*"DynFloatPropArray")["float"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (float)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (int16)
	{
		int16 TargetResult = 512;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynInt16PropArray.Empty();
			TestObj->DynInt16PropArray.AddZeroed(1);

			TestObj->DynInt16PropArray[0] = TargetResult;
		}

		bool bError = false;
		int16 Result;
		
		Result = (int16)((FVMReflection(TestObj)->*"DynInt16PropArray")["int16"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (int16)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (int64)
	{
		int64 TargetResult = 982987423;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynInt64PropArray.Empty();
			TestObj->DynInt64PropArray.AddZeroed(1);

			TestObj->DynInt64PropArray[0] = TargetResult;
		}

		bool bError = false;
		int64 Result;
		
		Result = (int64)((FVMReflection(TestObj)->*"DynInt64PropArray")["int64"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (int64)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (int8)
	{
		int8 TargetResult = 42;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynInt8PropArray.Empty();
			TestObj->DynInt8PropArray.AddZeroed(1);

			TestObj->DynInt8PropArray[0] = TargetResult;
		}

		bool bError = false;
		int8 Result;
		
		Result = (int8)((FVMReflection(TestObj)->*"DynInt8PropArray")["int8"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (int8)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (int32)
	{
		int32 TargetResult = 65538;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynIntPropArray.Empty();
			TestObj->DynIntPropArray.AddZeroed(1);

			TestObj->DynIntPropArray[0] = TargetResult;
		}

		bool bError = false;
		int32 Result;
		
		Result = (int32)((FVMReflection(TestObj)->*"DynIntPropArray")["int32"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (int32)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (uint16)
	{
		uint16 TargetResult = 1024;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynUInt16PropArray.Empty();
			TestObj->DynUInt16PropArray.AddZeroed(1);

			TestObj->DynUInt16PropArray[0] = TargetResult;
		}

		bool bError = false;
		uint16 Result;
		
		Result = (uint16)((FVMReflection(TestObj)->*"DynUInt16PropArray")["uint16"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (uint16)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (uint32)
	{
		uint32 TargetResult = 65539;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynUIntPropArray.Empty();
			TestObj->DynUIntPropArray.AddZeroed(1);

			TestObj->DynUIntPropArray[0] = TargetResult;
		}

		bool bError = false;
		uint32 Result;
		
		Result = (uint32)((FVMReflection(TestObj)->*"DynUIntPropArray")["uint32"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (uint32)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (uint64)
	{
		uint64 TargetResult = 89389732783ULL;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynUInt64PropArray.Empty();
			TestObj->DynUInt64PropArray.AddZeroed(1);

			TestObj->DynUInt64PropArray[0] = TargetResult;
		}

		bool bError = false;
		uint64 Result;
		
		Result = (uint64)((FVMReflection(TestObj)->*"DynUInt64PropArray")["uint64"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (uint64)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (UObject*)
	{
		UObject* TargetResult = AActor::StaticClass();
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynObjectPropArray.Empty();
			TestObj->DynObjectPropArray.AddZeroed(1);

			TestObj->DynObjectPropArray[0] = TargetResult;
		}

		bool bError = false;
		UObject* Result;
		
		Result = (UObject*)((FVMReflection(TestObj)->*"DynObjectPropArray")["UObject*"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (UObject*)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (FString)
	{
		FString TargetResult = TEXT("blah");
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynStringPropArray.Empty();
			TestObj->DynStringPropArray.AddZeroed(1);

			TestObj->DynStringPropArray[0] = TargetResult;
		}

		bool bError = false;
		FString Result;
		
		Result = (FString)((FVMReflection(TestObj)->*"DynStringPropArray")["FString"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (FString)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (FText)
	{
		FText TargetResult = FText::FromString(TEXT("Blahd"));
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynTextPropArray.Empty();
			TestObj->DynTextPropArray.AddZeroed(1);

			TestObj->DynTextPropArray[0] = TargetResult;
		}

		bool bError = false;
		FText Result;
		
		Result = (FText)((FVMReflection(TestObj)->*"DynTextPropArray")["FText"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (FText)"), (!bError && Result.EqualTo(TargetResult)));
	}

	// Dynamic array type verification (UClass*)
	{
		UClass* TargetResult = AActor::StaticClass();
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynClassPropArray.Empty();
			TestObj->DynClassPropArray.AddZeroed(1);

			TestObj->DynClassPropArray[0] = TargetResult;
		}

		bool bError = false;
		UClass* Result;
		
		Result = (UClass*)(UObject*)((FVMReflection(TestObj)->*"DynClassPropArray")["UClass*"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (UClass*)"), (!bError && Result == TargetResult));
	}

	// Dynamic array type verification (APawn*)
	{
		APawn* TargetResult = (APawn*)GetDefault<ADefaultPawn>();
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynPawnPropArray.Empty();
			TestObj->DynPawnPropArray.AddZeroed(1);

			TestObj->DynPawnPropArray[0] = TargetResult;
		}

		bool bError = false;
		APawn* Result;
		
		Result = (APawn*)(UObject*)((FVMReflection(TestObj)->*"DynPawnPropArray")["APawn*"][0], &bError);

		TestResults.Add(TEXT("Dynamic Array Type Verify (APawn*)"), (!bError && Result == TargetResult));
	}


	// Dynamic array adding
	{
		uint8 TargetResults[8] = {4, 45, 31, 67, 99, 104, 192, 30};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->DynBytePropArray.Empty();

		bool bError;
		TArray<uint8>* ArrayRef = (TArray<uint8>*)(FScriptArray*)((FVMReflection(TestObj)->*"DynBytePropArray")["uint8"], &bError);

		if (!bError && ArrayRef != NULL)
		{
			TArray<uint8>& TargetArray = *ArrayRef;

			TargetArray.AddZeroed(8);

			TargetArray[0] = TargetResults[0];
			TargetArray[1] = TargetResults[1];
			TargetArray[2] = TargetResults[2];
			TargetArray[3] = TargetResults[3];
			TargetArray[4] = TargetResults[4];
			TargetArray[5] = TargetResults[5];
			TargetArray[6] = TargetResults[6];
			TargetArray[7] = TargetResults[7];
		}

		TestResults.Add(TEXT("Dynamic Array Adding"), !bError && TestObj->DynBytePropArray.Num() == 8 &&
						(TestObj->DynBytePropArray[0] == TargetResults[0]) &&
						(TestObj->DynBytePropArray[1] == TargetResults[1]) &&
						(TestObj->DynBytePropArray[2] == TargetResults[2]) &&
						(TestObj->DynBytePropArray[3] == TargetResults[3]) &&
						(TestObj->DynBytePropArray[4] == TargetResults[4]) &&
						(TestObj->DynBytePropArray[5] == TargetResults[5]) &&
						(TestObj->DynBytePropArray[6] == TargetResults[6]) &&
						(TestObj->DynBytePropArray[7] == TargetResults[7]));
	}


	// Struct property reading
	{
		float TargetResult = 12.8f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->StructProp.X = TargetResult;
		}

		bool bError = false;
		float Result = (float)(FVMReflection(TestObj)->*"StructProp"->*"X", &bError);

		TestResults.Add(TEXT("Struct Reading"), (!bError && Result == TargetResult));
	}

	// Struct property writing
	{
		float TargetResult = 6.4f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->StructProp.Y = 12.8f;

		bool bError = false;
		float* Result = (float*)(FVMReflection(TestObj)->*"StructProp"->*"Y", &bError);

		if (!bError && Result != NULL)
		{
			*Result = TargetResult;
		}

		TestResults.Add(TEXT("Struct Writing"), (!bError && TestObj->StructProp.Y == TargetResult));
	}


	// Struct property casting
	{
		float TargetResult = 12.8f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->StructProp.X = 0.f;
		}

		bool bError = false;
		FVector* StructRef = (FVector*)(void*)((FVMReflection(TestObj)->*"StructProp")["FVector"], &bError);

		if (!bError && StructRef != NULL)
		{
			StructRef->X = TargetResult;
		}

		TestResults.Add(TEXT("Struct Casting"), (!bError && TestObj->StructProp.X == TargetResult));
	}


	// Struct static array reading
	{
		float TargetResult[2] = {12.8f, 83.2f};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->StructPropArray[0].X = TargetResult[0];
			TestObj->StructPropArray[1].X = TargetResult[1];
		}

		bool bError[2] = {false, false};
		float Results[2];
		
		Results[0] = (float)((FVMReflection(TestObj)->*"StructPropArray")["FVector"][0]->*"X", &bError[0]);
		Results[1] = (float)((FVMReflection(TestObj)->*"StructPropArray")["FVector"][1]->*"X", &bError[1]);

		TestResults.Add(TEXT("Struct Static Array Reading"), (!bError[0] && Results[0] == TargetResult[0]) &&
						(!bError[1] && Results[1] == TargetResult[1]));
	}

	// Struct static array writing
	{
		float TargetResult[2] = {6.4f, 82.3f};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->StructPropArray[0].Y = 12.8f;
		TestObj->StructPropArray[1].Y = 12.8f;

		bool bError[2] = {false, false};
		float* Results[2];
		
		Results[0] = (float*)((FVMReflection(TestObj)->*"StructPropArray")["FVector"][0]->*"Y", &bError[0]);
		Results[1] = (float*)((FVMReflection(TestObj)->*"StructPropArray")["FVector"][1]->*"Y", &bError[1]);

		if (!bError[0] && !bError[1] && Results[0] != NULL && Results[1] != NULL)
		{
			*Results[0] = TargetResult[0];
			*Results[1] = TargetResult[1];
		}

		TestResults.Add(TEXT("Struct Static Array Writing"), (!bError[0] && TestObj->StructPropArray[0].Y == TargetResult[0]) &&
						(!bError[1] && TestObj->StructPropArray[1].Y == TargetResult[1]));
	}


	// Struct static array casting
	{
		float TargetResult = 12.8f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->StructPropArray[1].X = 0.f;
		}

		bool bError = false;
		FVector* StructRef = (FVector*)(void*)((FVMReflection(TestObj)->*"StructPropArray")["FVector"][1], &bError);

		if (!bError && StructRef != NULL)
		{
			StructRef->X = TargetResult;
		}

		TestResults.Add(TEXT("Struct Static Array Casting"), (!bError && TestObj->StructPropArray[1].X == TargetResult));
	}



	// Struct dynamic array reading
	{
		float TargetResult[2] = {12.8f, 83.2f};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynStructPropArray.Empty();
			TestObj->DynStructPropArray.AddZeroed(2);

			TestObj->DynStructPropArray[0].X = TargetResult[0];
			TestObj->DynStructPropArray[1].X = TargetResult[1];
		}

		bool bError[2] = {false, false};
		float Results[2];
		
		Results[0] = (float)((FVMReflection(TestObj)->*"DynStructPropArray")["FVector"][0]->*"X", &bError[0]);
		Results[1] = (float)((FVMReflection(TestObj)->*"DynStructPropArray")["FVector"][1]->*"X", &bError[1]);

		TestResults.Add(TEXT("Struct Dynamic Array Reading"), (!bError[0] && Results[0] == TargetResult[0]) &&
						(!bError[1] && Results[1] == TargetResult[1]));
	}

	// Struct dynamic array writing
	{
		float TargetResult[2] = {6.4f, 82.3f};
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		TestObj->DynStructPropArray.Empty();
		TestObj->DynStructPropArray.AddZeroed(2);

		TestObj->DynStructPropArray[0].Y = 12.8f;
		TestObj->DynStructPropArray[1].Y = 12.8f;

		bool bError[2] = {false, false};
		float* Results[2];
		
		Results[0] = (float*)((FVMReflection(TestObj)->*"DynStructPropArray")["FVector"][0]->*"Y", &bError[0]);
		Results[1] = (float*)((FVMReflection(TestObj)->*"DynStructPropArray")["FVector"][1]->*"Y", &bError[1]);

		if (!bError[0] && !bError[1] && Results[0] != NULL && Results[1] != NULL)
		{
			*Results[0] = TargetResult[0];
			*Results[1] = TargetResult[1];
		}

		TestResults.Add(TEXT("Struct Dynamic Array Writing"), (!bError[0] && TestObj->DynStructPropArray[0].Y == TargetResult[0]) &&
						(!bError[1] && TestObj->DynStructPropArray[1].Y == TargetResult[1]));
	}


	// Struct dynamic array casting
	{
		float TargetResult = 12.8f;
		UVMTestClassA* TestObj = NewObject<UVMTestClassA>();

		if (TestObj != NULL)
		{
			TestObj->DynStructPropArray.Empty();
			TestObj->DynStructPropArray.AddZeroed(2);

			TestObj->DynStructPropArray[1].X = 0.f;
		}

		bool bError = false;
		FVector* StructRef = (FVector*)(void*)((FVMReflection(TestObj)->*"DynStructPropArray")["FVector"][1], &bError);

		if (!bError && StructRef != NULL)
		{
			StructRef->X = TargetResult;
		}

		TestResults.Add(TEXT("Struct Dynamic Array Casting"), (!bError && TestObj->DynStructPropArray[1].X == TargetResult));
	}




	// Verify the results
	for (TMap<FString, bool>::TConstIterator It(TestResults); It; ++It)
	{
		UNIT_LOG(ELogType::StatusImportant, TEXT("Test '%s' returned: %s"), *It.Key(), (It.Value() ? TEXT("Success") : TEXT("FAIL")));

		if (!It.Value())
		{
			VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
		}
	}


	if (VerificationState == EUnitTestVerification::Unverified)
	{
		VerificationState = EUnitTestVerification::VerifiedFixed;
	}

	return bSuccess;
}


UVMTestClassA::UVMTestClassA(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UVMTestClassB::UVMTestClassB(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

