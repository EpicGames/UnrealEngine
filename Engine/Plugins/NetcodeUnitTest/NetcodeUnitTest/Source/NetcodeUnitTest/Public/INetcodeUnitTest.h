// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * Module interface that plugins containing unit tests should use - to support hot reload properly
 */
class NETCODEUNITTEST_API FNUTModuleInterface : public IModuleInterface
{
private:
	/**
	 * Hide the default constructor, to force specification of module name
	 */
	FNUTModuleInterface()
		: ModuleName(nullptr)
	{
	}

public:
	/**
	 * Constructor which forces specification of the module name
	 */
	FNUTModuleInterface(const TCHAR* InModuleName)
		: ModuleName(InModuleName)
	{
	}

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;


private:
	/** The name of the module, specified by subclass */
	const TCHAR* ModuleName;
};


/**
 * Public interface for the NetcodeUnitTest module
 */
class INetcodeUnitTest : public FNUTModuleInterface
{
public:
	INetcodeUnitTest()
		: FNUTModuleInterface(TEXT("NetcodeUnitTest"))
	{
	}

	static inline INetcodeUnitTest& Get()
	{
		return FModuleManager::LoadModuleChecked<INetcodeUnitTest>("NetcodeUnitTest");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("NetcodeUnitTest");
	}
};

