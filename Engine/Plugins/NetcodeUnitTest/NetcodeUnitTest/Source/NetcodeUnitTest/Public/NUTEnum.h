// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "NetcodeUnitTest.h"

#include "NUTEnum.generated.h"

/**
 * Flags for configuring the minimal client, what parts of the netcode should be enabled etc.
 */
UENUM()
enum class EMinClientFlags : uint32
{
	None					= 0x00000000,	// No flags

	/** Minimal-client netcode functionality */
	AcceptActors			= 0x00000001,	// Whether or not to accept actor channels (acts as allowlist-only with NotifyAllowNetActor)
	AcceptRPCs				= 0x00000002,	// Whether or not to accept execution of any actor RPC's (they are all blocked by default)
	SendRPCs				= 0x00000004,	// Whether or not to allow RPC sending (NOT blocked by default, @todo JohnB: Add send hook)
	AcceptRepNotifies		= 0x00000008,	// Whether or not to accept execution of RepNotify's on Actors (blocked by default)
	SkipControlJoin			= 0x00000010,	// Whether or not to skip sending NMT_Join upon connect (or NMT_BeaconJoin for beacons)
	BeaconConnect			= 0x00000020,	// Whether or not to connect to the servers beacon (greatly limits the connection)

	/** Minimal-client events */
	NotifyNetActors			= 0x00000100,	// Whether or not to trigger a 'NotifyNetActor' event, AFTER creation of actor channel actor
	NotifyProcessNetEvent	= 0x00000200,	// Whether or not to trigger 'NotifyProcessNetEvent' for every client RPC function

	/** Debugging */
	DumpReceivedRaw			= 0x00001000,	// Whether or not to also hex-dump the raw packet receives to the log/log-window
	DumpSendRaw				= 0x00002000,	// Whether or not to also hex-dump the raw packet sends to the log/log-window
	DumpReceivedRPC			= 0x00004000,	// Whether or not to dump RPC receives (with LogNetTraffic, detects ProcessEvent RPC fail)
	DumpSendRPC				= 0x00008000	// Whether or not to dump RPC sends
};

// Required for bitwise operations with the above enum
ENUM_CLASS_FLAGS(EMinClientFlags);


/**
 * Flags for configuring how individual unit tests make use of the base client unit test framework.
 * NOTE: There is crossover between these flags, and EMinClientFlags.
 *
 *
 * Types of things these flags control:
 *	- Types of remote data which are accepted/denied (channel types, actors, RPC's)
 *		- IMPORTANT: This includes local setup of e.g. actor channels, and possibly execution of RPC's in local context,
 *			which risks undefined behaviour (which is why it is disabled by default - you have to know what you're doing)
 *
 *	- The prerequisites needed before executing the unit test (need a valid PlayerController? A particular actor?)
 *
 *	- Whether or not a server or another client is automatically launched
 *
 *	- Enabling other miscellaneous events, such as capturing raw packet data for debugging
 */
enum class EUnitTestFlags : uint32
{
	None					= 0x00000000,	// No flags

	/** Sub-process flags */
	LaunchServer			= 0x00000001,	// Whether or not to automatically launch a game server, for the unit test
	LaunchClient			= 0x00000002,	// Whether or not to automatically launch a full game client, which connects to the server

	/** Minimal-client netcode functionality */
	AcceptPlayerController	= 0x00000004,	// Whether or not to accept PlayerController creation
	BeaconConnect			= 0x00000008,	// Whether or not to connect to the servers beacon (greatly limits the connection)
	AutoReconnect			= 0x00000010,	// Whether or not to auto-reconnect on server disconnect (NOTE: Won't catch all disconnects)

	/** Unit test state-setup/requirements/prerequisites */
	RequirePlayerController	= 0x00000100,	// Whether or not to wait for the PlayerController, before triggering ExecuteClientUnitTest
	RequirePawn				= 0x00000200,	// Whether or not to wait for PlayerController's pawn, before ExecuteClientUnitTest
	RequirePlayerState		= 0x00000400,	// Whether or not to wait for PlayerController's PlayerState, before ExecuteClientUnitTest
	RequirePing				= 0x00000800,	// Whether or not to wait for a ping round-trip, before triggering ExecuteClientUnitTest
	RequireNUTActor			= 0x00001000,	// Whether or not to wait for the NUTActor, before triggering ExecuteClientUnitTest
	RequireBeacon			= 0x00002000,	// Whether or not to wait for beacon replication, before triggering ExecuteClientUnitTest
	RequireCustom			= 0x00004000,	// Whether or not ExecuteClientUnitTest will be executed manually, within the unit test

	RequirementsMask		= RequirePlayerController | RequirePawn | RequirePlayerState | RequirePing | RequireNUTActor |
								RequireBeacon | RequireCustom,

	/** Unit test error/crash detection */
	ExpectServerCrash		= 0x00100000,	// Whether or not this unit test will intentionally crash the server
	ExpectDisconnect		= 0x00200000,	// Whether or not this unit test will intentionally trigger a disconnect from the server

	/** Unit test error/crash detection debugging (NOTE: Don't use these in finalized unit tests, unit tests must handle all errors) */
	IgnoreServerCrash		= 0x00400000,	// Whether or not server crashes should be treated as a unit test failure
	IgnoreClientCrash		= 0x00800000,	// Whether or not client crashes should be treated as a unit test failure
	IgnoreDisconnect		= 0x01000000,	// Whether or not MinimalClient disconnects, should be treated as a unit test failure

	/** Unit test events */
	NotifyProcessEvent		= 0x02000000,	// Whether or not to trigger 'NotifyProcessEvent' for every executed non-RPC local function

	/** Debugging */
	CaptureReceivedRaw		= 0x04000000,	// Whether or not to capture raw (clientside) packet receives
	DumpControlMessages		= 0x08000000	// Whether or not to dump control channel messages, and their raw hex content
};

// Required for bitwise operations with the above enum
ENUM_CLASS_FLAGS(EUnitTestFlags);


/**
 * Used to get name values for the above enum
 */
FString GetUnitTestFlagName(EUnitTestFlags Flag);

/**
 * Converts any EUnitTestFlags values, to their EMinClientFlags equivalent
 */
EMinClientFlags FromUnitTestFlags(EUnitTestFlags Flags);


/**
 * Validates, both at compile time (template param) or at runtime (function param), that the specified EMinClientFlags flags are valid.
 *
 * When specifying the whole flag list in one assignment, do a compile time check using the template parameter.
 * When modifying a runtime-written flag list, do a runtime check on the final flag variable using the function parameter.
 *
 * @param CompileTimeFlags	The compile-time flag list to be checked
 * @param RuntimeFlags		The runtime flag list to be checked
 * @return					Returns the flags, so that validation can be done as they are assigned to a value
 */
template<EMinClientFlags CompileTimeFlags=EMinClientFlags::None>
EMinClientFlags ValidateMinFlags(EMinClientFlags RuntimeFlags=EMinClientFlags::None)
{
	// Implements the 'MinClientFlags' variable within 'Condition', for both static/compile-time checks, and runtime checks
	#define FLAG_ASSERT(Condition, Message) \
		{ \
			struct CompileTimeAssert \
			{ \
				void NeverCalled() \
				{ \
					constexpr EMinClientFlags MinClientFlags = CompileTimeFlags; \
					static_assert(CompileTimeFlags == EMinClientFlags::None || (Condition), Message); \
				} \
			}; \
			\
			if (RuntimeFlags != EMinClientFlags::None) \
			{ \
				EMinClientFlags MinClientFlags = RuntimeFlags; \
				UNIT_ASSERT(Condition); \
			} \
		}


	FLAG_ASSERT(!(MinClientFlags & EMinClientFlags::DumpReceivedRPC) || !!(MinClientFlags & EMinClientFlags::NotifyProcessNetEvent),
					"If you want to dump received RPC's, you need to hook NotifyProcessEvent");

	FLAG_ASSERT(!(MinClientFlags & EMinClientFlags::AcceptRPCs) || !!(MinClientFlags & EMinClientFlags::AcceptActors),
					"You can't accept RPC's, without accepting actors");

	FLAG_ASSERT(!(MinClientFlags & EMinClientFlags::NotifyNetActors) || !!(MinClientFlags & EMinClientFlags::AcceptActors),
					"You can't get net actor notifications, unless you accept actors");

	#undef FLAG_ASSERT

	return CompileTimeFlags | RuntimeFlags;
}


