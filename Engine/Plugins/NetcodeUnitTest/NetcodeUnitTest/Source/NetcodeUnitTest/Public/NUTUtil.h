// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "GameFramework/Actor.h"
#include "NetcodeUnitTest.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Misc/OutputDeviceError.h"


// Forward declarations
class UNetDriver;
class UPackage;
struct FStackTracker;
class FOutputDeviceFile;
class UUnitTest;

// @todo #JohnBRefactor: Adjust all of these utility .h files, so that they implement code in .cpp, as these probably slow down compilation


/**
 * Output device for allowing quick/dynamic creation of a customized output device, using lambda's passed to delegates
 */
class FDynamicOutputDevice : public FOutputDevice
{
public:
	FDynamicOutputDevice()
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		OnSerialize.Broadcast(V, Verbosity, Category);
	}

	virtual void Flush() override
	{
		OnFlush.Broadcast();
	}

	virtual void TearDown() override
	{
		OnTearDown.Broadcast();
	}

public:
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSerialize, const TCHAR*, ELogVerbosity::Type, const FName&);

	DECLARE_MULTICAST_DELEGATE(FOnFlushOrTearDown);


	FOnSerialize OnSerialize;

	FOnFlushOrTearDown OnFlush;

	FOnFlushOrTearDown OnTearDown;
};

/**
 * Output device for hijacking/hooking an existing output device (e.g. to hijack GError, to block specific asserts)
 * Inherit this class, to implement desired hook behaviour in subclass
 */
class FHookOutputDevice : public FOutputDeviceError
{
public:
	FHookOutputDevice()
		: OriginalDevice(nullptr)
	{
	}

	void HookDevice(FOutputDeviceError* OldDevice)
	{
		check(OriginalDevice == nullptr);

		OriginalDevice = OldDevice;
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (OriginalDevice != nullptr)
		{
			OriginalDevice->Serialize(V, Verbosity, Category);
		}
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time) override
	{
		if (OriginalDevice != nullptr)
		{
			OriginalDevice->Serialize(V, Verbosity, Category, Time);
		}
	}

	virtual void Flush() override
	{
		if (OriginalDevice != nullptr)
		{
			OriginalDevice->Flush();
		}
	}

	virtual void TearDown() override
	{
		if (OriginalDevice != nullptr)
		{
			OriginalDevice->TearDown();
		}
	}

	virtual void Dump(class FArchive& Ar) override
	{
		if (OriginalDevice != nullptr)
		{
			OriginalDevice->Dump(Ar);
		}
	}

	virtual bool IsMemoryOnly() const override
	{
		bool bReturnVal = false;

		if (OriginalDevice != nullptr)
		{
			bReturnVal = OriginalDevice->IsMemoryOnly();
		}

		return bReturnVal;
	}

	virtual bool CanBeUsedOnAnyThread() const override
	{
		bool bReturnVal = false;

		if (OriginalDevice != nullptr)
		{
			bReturnVal = OriginalDevice->CanBeUsedOnAnyThread();
		}

		return bReturnVal;
	}

	virtual void HandleError() override
	{
		if (OriginalDevice != nullptr)
		{
			OriginalDevice->HandleError();
		}
	}


private:
	/** The original output device */
	FOutputDeviceError* OriginalDevice;
};


/**
 * Output device for replacing GError, and catching specific asserts so they don't crash the game.
 */
class FAssertHookDevice : public FHookOutputDevice
{
public:
	FAssertHookDevice()
	{
	}

	/**
	 * Adds a string to the list of disabled asserts - does a partial match when watching for the disabled assert
	 */
	static void AddAssertHook(FString Assert);

	FORCEINLINE bool IsAssertDisabled(const TCHAR* V)
	{
		bool bReturnVal = false;

		for (auto CurEntry : DisabledAsserts)
		{
			if (FCString::Stristr(V, *CurEntry) != nullptr)
			{
				bReturnVal = true;
				break;
			}
		}

		if (bReturnVal)
		{
			UE_LOGF(LogUnitTest, Log, "Blocking disabled assert: %ls", V);
		}

		return bReturnVal;
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (!IsAssertDisabled(V))
		{
			FHookOutputDevice::Serialize(V, Verbosity, Category);
		}
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time) override
	{
		if (!IsAssertDisabled(V))
		{
			FHookOutputDevice::Serialize(V, Verbosity, Category, Time);
		}
	}

public:
	/** List of disabled asserts */
	TArray<FString> DisabledAsserts;
};


// Hack for accessing private members in various Engine classes

// This template, is used to link an arbitrary class member, to the GetPrivate function
template<typename Accessor, typename Accessor::Member Member> struct AccessPrivate
{
	friend typename Accessor::Member GetPrivate(Accessor InAccessor)
	{
		return Member;
	}
};


// Need to define one of these, for every member you want to access (preferably in the .cpp) - for example:
#if 0
// This is used to aid in linking one member in FStackTracker, to the above template
struct FStackTrackerbIsEnabledAccessor
{
	using Member = bool FStackTracker::*;

	friend Member GetPrivate(FStackTrackerbIsEnabledAccessor);
};

// These combine the structs above, with the template for accessing private members, pointing to the specified member
template struct AccessPrivate<FStackTrackerbIsEnabledAccessor, &FStackTracker::bIsEnabled>;
#endif

// Example for calling private functions
#if 0
// Used, in combination with another template, for accessing private/protected members of classes
struct AShooterCharacterServerEquipWeaponAccessor
{
	using Member = void (AShooterCharacter::*)(AShooterWeapon* Weapon);

	friend Member GetPrivate(AShooterCharacterServerEquipWeaponAccessor);
};

// Combines the above struct, with the template used for accessing private/protected members
template struct AccessPrivate<AShooterCharacterServerEquipWeaponAccessor, &AShooterCharacter::ServerEquipWeapon>;

// To call private function:
//	(GET_PRIVATE(AShooterCharacter, CurChar, ServerEquipWeapon))(AShooterWeapon::StaticClass());
#endif

/**
 * Defines a class and template specialization, for a variable, needed for use with the GET_PRIVATE hook below
 *
 * @param InClass		The class being accessed (not a string, just the class, i.e. FStackTracker)
 * @param VarName		Name of the variable being accessed (again, not a string)
 * @param VarType		The type of the variable being accessed
 */
#define IMPLEMENT_GET_PRIVATE_VAR(InClass, VarName, VarType) \
	struct InClass##VarName##Accessor \
	{ \
		using Member = VarType InClass::*; \
		\
		friend Member GetPrivate(InClass##VarName##Accessor); \
	}; \
	\
	template struct AccessPrivate<InClass##VarName##Accessor, &InClass::VarName>;

// @todo #JohnB: Unfortunately, this broke in VS2015 for functions, as '&InClass::FuncName' gives an error;
//					strictly speaking, this falls within the C++ standard and should work, so perhaps make a bug report
//					See SLogWidget.cpp, for an alternative way of hack-accessing private members, using template specialization
#if 0
/**
 * Defines a class and template specialization, for a function, needed for use with the GET_PRIVATE hook below
 *
 * @param InClass		The class being accessed (not a string, just the class, i.e. FStackTracker)
 * @param FuncName		Name of the function being accessed (again, not a string)
 * @param FuncRet		The return type of the function
 * @param FuncParms		The function parameters
 * @param FuncModifier	(Optional) Modifier placed after the function - usually 'const'
 */
#define IMPLEMENT_GET_PRIVATE_FUNC_CONST(InClass, FuncName, FuncRet, FuncParms, FuncModifier) \
	struct InClass##FuncName##Accessor \
	{ \
		using Member = FuncRet (InClass::*)(FuncParms) FuncModifier; \
		\
		friend Member GetPrivate(InClass##FuncName##Accessor); \
	}; \
	\
	template struct AccessPrivate<InClass##FuncName##Accessor, &InClass::FuncName>;

#define IMPLEMENT_GET_PRIVATE_FUNC(InClass, FuncName, FuncRet, FuncParms) \
	IMPLEMENT_GET_PRIVATE_FUNC_CONST(InClass, FuncName, FuncRet, FuncParms, ;)
#endif


/**
 * A macro for tidying up accessing of private members, through the above code
 *
 * @param InClass		The class being accessed (not a string, just the class, i.e. FStackTracker)
 * @param InObj			Pointer to an instance of the specified class
 * @param MemberName	Name of the member being accessed (again, not a string)
 * @return				The value of the member 
 */
#define GET_PRIVATE(InClass, InObj, MemberName) (*InObj).*GetPrivate(InClass##MemberName##Accessor())

/**
 * Redundant version of the above, for emphasizing the ability to assign a writable reference which can modify the original value
 */
#define GET_PRIVATE_REF(InClass, InObj, MemberName) GET_PRIVATE(InClass, InObj, MemberName)


// @todo #JohnB: Restore if fixed in VS2015
#if 0
// Version of above, for calling private functions
#define CALL_PRIVATE(InClass, InObj, MemberName) (GET_PRIVATE(InClass, InObj, MemberName))
#endif


/**
 * Defines a class used for hack-accessing protected functions, through the CALL_PROTECTED macro below
 *
 * @param InClass		The class being accessed (not a string, just the class, i.e. FStackTracker)
 * @param FuncName		Name of the function being accessed (again, not a string)
 * @param FuncRet		The return type of the function
 * @param FuncParms		The function parameters
 * @param FuncParmNames	The names of the function parameters, as passed from one function to another
 * @param FuncModifier	(Optional) Modifier placed after the function - usually 'const'
 */
#define IMPLEMENT_GET_PROTECTED_FUNC_CONST(InClass, FuncName, FuncRet, FuncParms, FuncParmNames, FuncModifier) \
	struct InClass##FuncName##Accessor : public InClass \
	{ \
		FORCEINLINE FuncRet FuncName##Accessor(FuncParms) FuncModifier \
		{\
			return FuncName(FuncParmNames); \
		} \
	};

#define IMPLEMENT_GET_PROTECTED_FUNC(InClass, FuncName, FuncRet, FuncParms, FuncParmNames) \
	IMPLEMENT_GET_PROTECTED_FUNC_CONST(InClass, FuncName, FuncRet, FuncParms, FuncParmNames,)

// Version of GET_PRIVATE, for calling protected functions
#define CALL_PROTECTED(InClass, InObj, MemberName) ((InClass##MemberName##Accessor*)&(*InObj))->MemberName##Accessor



/**
 * General utility functions
 */
struct NUTUtil
{
	/**
	 * Returns the currently active net driver (either pending or one for current level)
	 *
	 * @return	The active net driver
	 */
	static inline UNetDriver* GetActiveNetDriver(UWorld* InWorld)
	{
		UNetDriver* ReturnVal = NULL;
		FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(InWorld);

		if (WorldContext != NULL && WorldContext->PendingNetGame != NULL && WorldContext->PendingNetGame->NetDriver != NULL)
		{
			ReturnVal = WorldContext->PendingNetGame->NetDriver;
		}
		else if (InWorld != NULL)
		{
			ReturnVal = InWorld->NetDriver;
		}

		return ReturnVal;
	}

	static inline UWorld* GetPrimaryWorld()
	{
		UWorld* ReturnVal = NULL;

		if (GEngine != NULL)
		{
			for (auto It=GEngine->GetWorldContexts().CreateConstIterator(); It; ++It)
			{
				const FWorldContext& Context = *It;

				if ((Context.WorldType == EWorldType::Game || Context.WorldType == EWorldType::PIE) && Context.World())
				{
					ReturnVal = Context.World();
					break;
				}
			}
		}

		return ReturnVal;
	}

	/**
	 * Outputs a full list of valid unit test class defaults, representing all unit tests which can be executed
	 *
	 * @param OutUnitTestClassDefaults	The array that unit test class defaults should be output to
	 */
	static void GetUnitTestClassDefList(TArray<UUnitTest*>& OutUnitTestClassDefaults);

	/**
	 * Takes a list of unit test class defaults, and reorders them in a more readable way, based on type and implementation date
	 *
	 * @param InUnitTestClassDefaults	The array of unit test class defaults to reorder
	 */
	static void SortUnitTestClassDefList(TArray<UUnitTest*>& InUnitTestClassDefaults);


	/**
	 * Overridden parse function, for supporting escaped quotes, e.g: -UnitTestServerParms="-LogCmds=\"LogNet all\""
	 */
	static bool ParseValue(const TCHAR* Stream, const TCHAR* Match, TCHAR* Value, int32 MaxLen, bool bShouldStopOnComma=true)
	{
		bool bReturnVal = false;
		const TCHAR* Found = FCString::Strifind(Stream, Match);

		if (Found != NULL)
		{
			const TCHAR* Start = Found + FCString::Strlen(Match);

			// Check for quoted arguments' string with spaces
			// -Option="Value1 Value2"
			//         ^~~~Start
			bool bArgumentsQuoted = *Start == '"';

			// Number of characters we can look back from found looking for first parenthesis.
			uint64 AllowedBacktraceCharactersCount = Found - Stream;

			// Check for fully quoted string with spaces
			bool bFullyQuoted = 
				// "Option=Value1 Value2"
				//  ^~~~Found
				(AllowedBacktraceCharactersCount > 0 && (*(Found - 1) == '"'))
				// "-Option=Value1 Value2"
				//   ^~~~Found
				|| (AllowedBacktraceCharactersCount > 1 && ((*(Found - 1) == '-') && (*(Found - 2) == '"')));

			if (bArgumentsQuoted || bFullyQuoted)
			{
				// Skip quote character if only params were quoted.
				int32 QuoteCharactersToSkip = bArgumentsQuoted ? 1 : 0;
				FCString::Strncpy(Value, Start + QuoteCharactersToSkip, MaxLen);

				Value[MaxLen-1] = 0;

				TCHAR* EndQuote = FCString::Strstr(Value, TEXT("\x22"));
				TCHAR* EscapedQuote = FCString::Strstr(Value, TEXT("\\\x22"));
				bool bContainsEscapedQuotes = EscapedQuote != NULL;

				// JohnB: Keep iterating until we've moved past all escaped quotes
				while (EndQuote != NULL && EscapedQuote != NULL && EscapedQuote < EndQuote)
				{
					TCHAR* Temp = EscapedQuote + 2;

					EndQuote = FCString::Strstr(Temp, TEXT("\x22"));
					EscapedQuote = FCString::Strstr(Temp, TEXT("\\\x22"));
				}

				if (EndQuote != NULL)
				{
					*EndQuote = 0;
				}

				// JohnB: Fixup all escaped quotes
				if (bContainsEscapedQuotes)
				{
					EscapedQuote = FCString::Strstr(Value, TEXT("\\\x22"));

					while (EscapedQuote != NULL)
					{
						TCHAR* Temp = EscapedQuote;

						while (true)
						{
							Temp[0] = Temp[1];

							if (*(++Temp) == '\0')
							{
								break;
							}
						}

						EscapedQuote = FCString::Strstr(Value, TEXT("\\\x22"));
					}
				}
			}
			else
			{
				// Non-quoted string without spaces.
				FCString::Strncpy(Value, Start, MaxLen);

				Value[MaxLen-1] = 0;

				TCHAR* Temp = FCString::Strstr(Value, TEXT(" "));
				
				if (Temp != NULL)
				{
					*Temp = 0;
				}

				Temp = FCString::Strstr(Value, TEXT("\r"));
				
				if (Temp != NULL)
				{
					*Temp = 0;
				}

				Temp = FCString::Strstr(Value, TEXT("\n"));
				
				if (Temp != NULL)
				{
					*Temp = 0;
				}

				Temp = FCString::Strstr(Value, TEXT("\t"));
				
				if (Temp != NULL)
				{
					*Temp = 0;
				}


				if (bShouldStopOnComma)
				{
					Temp = FCString::Strstr(Value, TEXT(","));

					if (Temp != NULL)
					{
						*Temp = 0;
					}
				}
			}

			bReturnVal = true;
		}

		return bReturnVal;
	}

	/**
	 * Overridden parse function, for supporting escaped quotes
	 */
	static bool ParseValue(const TCHAR* Stream, const TCHAR* Match, FString& Value, bool bShouldStopOnComma=true)
	{
		bool bReturnVal = false;
		TCHAR Temp[4096] = TEXT("");

		if (ParseValue(Stream, Match, Temp, UE_ARRAY_COUNT(Temp), bShouldStopOnComma))
		{
			Value = Temp;
			bReturnVal = true;
		}

		return bReturnVal;
	}

	/**
	 * Parses the value of specific enum, from the specified commandline stream
	 *
	 * @param Stream	The stream to parse from (typically FCommandline::Get())
	 * @param Match		The commandline option to match as the key specifying the value
	 * @param EnumName	The name of the enum type
	 * @param Value		Outputs the integer value of the enum, as specified on the commandline
	 * @return			Whether or not an enum value was parsed successfully
	 */
	static bool ParseEnum(const TCHAR* Stream, const TCHAR* Match, const TCHAR* EnumName, uint64& Value);


	/**
	 * Puts out a log message to FOutputDeviceFile, with a special category prefix added.
	 *
	 * For example:
	 *	[2017.02.27-15.09.15:999][  0][SpecialCategory]LogUnitTest: LogMessage
	 *
	 * @param Ar				The FOutputDeviceFile archive to write to
	 * @param SpecialCategory	The special category prefix to be added to the log message
	 * @param Data				The log message
	 * @param Verbosity			The verbosity of the log message
	 * @param Category			The category of the log message
	 */
	static void SpecialLog(FOutputDeviceFile* Ar, const TCHAR* SpecialCategory, const TCHAR* Data, ELogVerbosity::Type Verbosity,
							const FName& Category);

	/**
	 * Get a module name from any UPackage - if it has one
	 *
	 * @param Package			The package to be linked to a module
	 * @param bVerifyModule		Whether or not to verify that the module is loaded (specify false, when using with Hot Reload code)
	 * @return					The name of the module
	 */
	static FString GetPackageModule(UPackage* Package, bool bVerifyModule=true);
};

