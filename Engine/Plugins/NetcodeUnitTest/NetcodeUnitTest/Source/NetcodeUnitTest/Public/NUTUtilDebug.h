// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "NetcodeUnitTest.h"
#include "NUTUtil.h"
#include "Containers/StackTracker.h"


// Forward declarations
class FLogStackTraceManager;
class FStackTraceManager;
class UClientUnitTest;
enum class ELogTraceFlags : uint16;

namespace UE::NUT
{
	class FScopedIncrementTraceIgnoreDepth;
	class FLogHookManager;
	class FLogTraceManager;
	class FLogCommandManager;
}


// Globals

/** Provides a globally accessible trace manager, for easy access to stack trace debugging */
extern FStackTraceManager* GTraceManager;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/** Log hook for managing tying of log entry detection to the trace manager */
extern FLogStackTraceManager* GLogTraceManager;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

namespace UE::NUT
{
	/** Generic string matching log hook, for executing code upon certain log entries */
	extern UE::NUT::FLogHookManager* GLogHookManager;

	/** Log hook for managing tying of log entry detection to the trace manager */
	extern UE::NUT::FLogTraceManager* GLogTrace;

	/** Log hook for executing console commands for certain log entries */
	extern UE::NUT::FLogCommandManager* GLogCommandManager;
}


// Enable access to FStackTracker.bIsEnabled
IMPLEMENT_GET_PRIVATE_VAR(FStackTracker, bIsEnabled, bool);

// The depth of stack traces, which the stack tracker should ignore by default
#define TRACE_IGNORE_DEPTH 7


/**
 * Provides a globally-accessible wrapper for the Exec function, which all modules can use (including those that don't inherit Engine),
 * for executing console commands - useful for debugging e.g. pre-Engine netcode (such as the PacketHandler code).
 */

extern "C" DLLEXPORT bool GGlobalExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

// Use this following code snippet, to execute the above function.
#if 0
	FModuleStatus MS;
	void* DH = FModuleManager::Get().QueryModule(TEXT("NetcodeUnitTest"), MS) ? FPlatformProcess::GetDllHandle(*MS.FilePath) : nullptr;
	void* ExecPtr = DH != nullptr ? FPlatformProcess::GetDllExport(DH, TEXT("GGlobalExec")) : nullptr;

	if (ExecPtr != nullptr)
	{
		((bool (*)(void* InWorld, const TCHAR* Cmd, FOutputDevice& Ar))ExecPtr)
			(nullptr, TEXT("Place console command here."), *GLog);
	}
#endif


/**
 * A class for enabling verbose log message categories, within a particular code scope (disabled when going out of scope).
 * NOTE: If you are logging any kind of net-related log messages, specify a unit test (even if you aren't doing remote logging)
 *
 * Also supports remote (server) logging, for net functions executed within the current code scope
 * (causes net packets to be flushed both upon entering the current scope, and when exiting it - required for correct log timing).
 *
 * NOTE: If you are trying to catch remote log messages deep within the internal game netcode, 
 *			then this class may not be appropriate, as remote logging passes through the netcode (YMMV)
 */
class NETCODEUNITTEST_API FScopedLog
{
protected:
	FScopedLog()
		: LogCategories()
		, UnitTest(nullptr)
		, bRemoteLogging(false)
		, bSuppressLogging(false)
	{
	}

public:
	/**
	 * Constructor used for setting up the type of logging that is done.
	 *
	 * @param InLogCategories	The list of log categories to be enabled
	 * @param InUnitTest		When tracking netcode-related logs, or doing remote logging, specify the client unit test here
	 * @param bInRemoteLogging	Whether or not to enable logging on the remote server
	 */
	FORCEINLINE FScopedLog(const TArray<FString>& InLogCategories, UClientUnitTest* InUnitTest=nullptr, bool bInRemoteLogging=false)
		: FScopedLog()
	{
		InternalConstruct(InLogCategories, InUnitTest, bInRemoteLogging);
	}

	// As above, but for a single log category
	FORCEINLINE FScopedLog(const FString InLogCategory, UClientUnitTest* InUnitTest=nullptr, bool bInRemoteLogging=false)
		: FScopedLog()
	{
		TArray<FString> TempLogCategories;
		TempLogCategories.Add(InLogCategory);

		InternalConstruct(TempLogCategories, InUnitTest, bInRemoteLogging);
	}

protected:
	void InternalConstruct(const TArray<FString>& InLogCategories, UClientUnitTest* InUnitTest, bool bInRemoteLogging);

public:
	~FScopedLog();


protected:
	/** The list of unsuppressed log messages */
	TArray<FString> LogCategories;

	/** Stores a reference to the unit test doing the logging, if specified */
	UClientUnitTest* UnitTest;

	/** Whether or not this is also controlling remote logging as well */
	bool bRemoteLogging;

	/** Whether or not to suppress instead of enable logging */
	bool bSuppressLogging;
};

/**
 * Version of FScopedLog, for suppressing instead of enabling log entries
 */
class FScopedLogSuppress : public FScopedLog
{
public:
	FORCEINLINE FScopedLogSuppress(const TArray<FString>& InLogCategories, UClientUnitTest* InUnitTest=nullptr,
									bool bInRemoteLogging=false)
		: FScopedLog()
	{
		bSuppressLogging = true;

		InternalConstruct(InLogCategories, InUnitTest, bInRemoteLogging);
	}

	FORCEINLINE FScopedLogSuppress(const FString InLogCategory, UClientUnitTest* InUnitTest=nullptr, bool bInRemoteLogging=false)
		: FScopedLog()
	{
		TArray<FString> TempLogCategories;
		TempLogCategories.Add(InLogCategory);

		bSuppressLogging = true;

		InternalConstruct(TempLogCategories, InUnitTest, bInRemoteLogging);
	}
};

/**
 * Version of FScopedLog, for scoped logging of all netcode-related logs
 */
class NETCODEUNITTEST_API FScopedLogNet : public FScopedLog
{
public:
	FORCEINLINE FScopedLogNet(UClientUnitTest* InUnitTest, bool bInRemoteLogging=false)
	{
		TArray<FString> TempLogCategories;

		// @todo #JohnBDebug: See if there are any other good net categories to add here
		TempLogCategories.Add(TEXT("LogNet"));
		TempLogCategories.Add(TEXT("LogRep"));
		TempLogCategories.Add(TEXT("LogNetTraffic"));
		TempLogCategories.Add(TEXT("LogRepTraffic"));
		TempLogCategories.Add(TEXT("LogNetSerialization"));
		TempLogCategories.Add(TEXT("LogNetPackageMap"));
		TempLogCategories.Add(TEXT("LogNetPlayerMovement"));
		TempLogCategories.Add(TEXT("LogNetDormancy"));
		TempLogCategories.Add(TEXT("LogProperty"));

		InternalConstruct(TempLogCategories, InUnitTest, bInRemoteLogging);
	}
};


// @todo #JohnB: When you continue implementing this, as a part of the ProcessEvent stack trace feature below,
//					merge this class with the very similar 'FProcessEventHook' class in NUTUtilNet.h,
//					then make the stack trace hook use that.
#if 0
#if !UE_BUILD_SHIPPING
/**
 * Base class for transparently hooking ProcessEvent
 */
class NETCODEUNITTEST_API FProcessEventHookBase
{
public:
	FProcessEventHookBase();

	virtual ~FProcessEventHookBase();

	// NOTE: Technically I 'could' allow the virtual override to return bool as well,
	//			but there is too great a risk of breaking something accidentally, so will not add until needed
	bool InternalProcessEventHook(AActor* Actor, UFunction* Function, void* Parameters);

	/**
	 * Implement the ProcessEvent hook, by overriding this in a subclass
	 *
	 * @param Actor			The actor the function is being called on
	 * @param Function		The function being called
	 * @param Parameters	The raw function parameters
	 */
	virtual void ProcessEventHook(AActor* Actor, UFunction* Function, void* Parameters)
		PURE_VIRTUAL(FProcessEventHookBase::ProcessEventHook(),);

protected:
	/** If a 'Actor::ProcessEventDelegate' value was already set, this caches it so it can be transparently hooked and restored later */
	FOnProcessEvent		OrigEventHook;
};
#endif
#endif



// @todo #JohnB: Reimplement this, by refactoring the above commented class, when this debug feature is next needed
#if 0
/**
 * A class for hooking and logging all ProcessEvent calls, within a particular code scope, similar to the above code
 */
class NETCODEUNITTEST_API FScopedProcessEventLog : FProcessEventHookBase
{
public:
	FScopedProcessEventLog()
		: FProcessEventHookBase()
	{
	}

	virtual void ProcessEventHook(AActor* Actor, UFunction* Function, void* Parameters) override;
};
#endif


/**
 * A class for dumping a stack trace, upon encountering a specific piece of code
 */
class FNUTStackTrace
{
	friend FStackTraceManager;
	friend UE::NUT::FScopedIncrementTraceIgnoreDepth;

public:
	/**
	 * Constructs the debug stack trace
	 *
	 * @param InTraceName	A human readable trace name for logging/debugging/tracking
	 */
	FNUTStackTrace(FString InTraceName);

	/**
	 * Destructor
	 */
	~FNUTStackTrace();


	/**
	 * Enable stack tracking
	 */
	void Enable();

	/**
	 * Disable stack tracking (past traces are still kept in tracking, but no new ones are added until re-enabled)
	 */
	void Disable();

	/**
	 * Adds a new trace to the stack tracker (optionally dumping to log at the same time)
	 *
	 * @param bLogAdd	Whether or not to also log this new trace being added
	 */
	void AddTrace(bool bLogAdd=false);

	/**
	 * Dumps accumulated stack traces
	 *
	 * @param bKeepTraceHistory		Whether or not to keep the trace history after dumping it (defaults to false)
	 */
	void Dump(bool bKeepTraceHistory=false);


	/**
	 * Whether or not the stack tracker, is currently tracking
	 */
	FORCEINLINE bool IsTrackingEnabled()
	{
		return GET_PRIVATE(FStackTracker, &Tracker, bIsEnabled);
	}

protected:
	/** The human-readable name to provide for this stack trace */
	FString TraceName;

	/** The stack tracker associated with this debug trace */
	FStackTracker Tracker;

private:
	/** Additional increment for the stack trace ignore depth, as set by FScopedIncrementTraceIgnoreDepth */
	int32 IgnoreDepthOffset;
};


// @todo #JohnBDocMain: Add documentation for all of the stack trace stuff, to the primary NetcodeUnitTest documentation

/**
 * Manager for handling multiple debug stack traces on-the-fly, and allowing abstraction of stack traces,
 * so you don't have to manually handle FNUTStackTrace objects (which can be complicated/bug-prone).
 *
 * This is a more intuitive way of handling tracing, you just use a call to 'GTraceManager->AddTrace' wherever needed,
 * and add calls to 'Enable'/'Disable' whenever you want to accept/ignore AddTrace calls - then 'Dump' to see the results.
 * No messing with managing instances of above objects.
 *
 * This also hooks into console commands as well, allowing it to be used throughout the engine, instead of depending on this module.
 * See the documentation for the 'StackTrace' command in UnitTestManager.cpp.
 */
class FStackTraceManager
{
	friend UE::NUT::FScopedIncrementTraceIgnoreDepth;

public:
	/**
	 * Base constructor
	 */
	FStackTraceManager();

	/**
	 * Destructor
	 */
	~FStackTraceManager();


	/**
	 * Passes on an 'Enable' call, to the specified stack trace
	 *
	 * @param TraceName		The name of the trace
	 */
	void Enable(FString TraceName);

	/**
	 * Passes on a 'Disable' call, to the specified stack trace (if it exists)
	 *
	 * @param TraceName		The name of the trace
	 */
	void Disable(FString TraceName);

	/**
	 * Adds a new stack trace, to the specified trace, optionally logging/dumping in the process
	 *
	 * @param TraceName			The name of the trace
	 * @param bDump				Whether or not to dump the trace as well (does not remove trace from tracking, unlike 'Dump' function)
	 * @param bStartDisabled	Whether or not this trace should start as disabled, until 'Enable' is called for it
	 */
	void AddTrace(FString TraceName, bool bLogAdd=false, bool bDump=false, bool bStartDisabled=false);

	/**
	 * Dumps accumulated stack traces, and removes from tracking (unless otherwise specified)
	 *
	 * @param TraceName				The name of the trace
	 * @param bKeepTraceHistory		Whether or not to keep the trace history after dumping it (defaults to false)
	 * @param bKeepTracking			Whether or not to keep on tracking this particular trace name (defaults to true)
	 */
	void Dump(FString TraceName, bool bKeepTraceHistory=false, bool bKeepTracking=true);

	/**
	 * Clears the specified trace from tracking
	 *
	 * @param TraceName		The name of the trace
	 */
	void Clear(FString TraceName);

	/**
	 * Dumps accumulated stack traces, for all tracked traces
	 *
	 * @param bKeepTraceHistory		Whether or not to keep the trace history after dumping (defaults to false)
	 * @param bKeepTracking			Whether or not to keep on tracking dumped traces (defaults to true)
	 */
	void DumpAll(bool bKeepTraceHistory=false, bool bKeepTracking=true);


	/**
	 * Performs a once-off stack trace, with no tracking (but if there is already a trace active with this name,
	 * respect its 'enabled' status)
	 *
	 * @param TraceName		The name of the trace
	 */
	void TraceAndDump(FString TraceName);


	/**
	 * Whether or not a trace of this name is present
	 *
	 * @param TraceName		The name of the trace
	 */
	FORCEINLINE bool ContainsTrace(FString TraceName)
	{
		return Traces.Contains(TraceName);
	}

protected:
	/**
	 * Gets the trace of the specified name, or NULL if it doesn't exist
	 *
	 * @param TraceName		The name of the trace
	 * @return				The trace or NULL if it doesn't exist
	 */
	FORCEINLINE FNUTStackTrace* GetTrace(FString TraceName)
	{
		FNUTStackTrace** MapPtr = Traces.Find(TraceName);

		return (MapPtr != NULL ? *MapPtr : NULL);
	}

	/**
	 * Gets or creates a trace, of the specified name
	 *
	 * @param TraceName		The name of the trace
	 * @param bOutNewTrace	Optionally, specifies whether this trace is newly created
	 */
	FORCEINLINE FNUTStackTrace* GetOrCreateTrace(FString TraceName, bool* bOutNewTrace=NULL)
	{
		FNUTStackTrace* ReturnVal = GetTrace(TraceName);

		if (bOutNewTrace != NULL)
		{
			*bOutNewTrace = false;
		}

		if (ReturnVal == NULL)
		{
			FNUTStackTrace* NewTrace = new FNUTStackTrace(TraceName);

			NewTrace->IgnoreDepthOffset = IgnoreDepthOffset;
			ReturnVal = Traces.Add(TraceName, NewTrace);

			if (bOutNewTrace)
			{
				*bOutNewTrace = true;
			}
		}

		return ReturnVal;
	}

protected:
	/** A map of active debug stack traces */
	TMap<FString, FNUTStackTrace*> Traces;

private:
	/** Additional increment for the stack trace ignore depth, as set by FScopedIncrementTraceIgnoreDepth */
	int32 IgnoreDepthOffset;
};


namespace UE::NUT
{

/**
 * Used for adjusting the depth of ignore stack trace lines in FStackTraceManager
 */
class FScopedIncrementTraceIgnoreDepth final
{
public:
	/**
	 * Base constructor
	 *
	 * @param InManager				The stack trace manager whose ignore depth is being adjusted
	 * @param InIgnoreIncrement		The amount by which the stack trace managers ignore deoth should be incremented
	 */
	FScopedIncrementTraceIgnoreDepth(FStackTraceManager* InManager, int32 InIgnoreIncrement);

	~FScopedIncrementTraceIgnoreDepth();

private:
	FStackTraceManager* Manager = nullptr;

	int32 IgnoreIncrement = 0;
};

/**
 * Specifies the type of log hook (e.g. type of string matching to use)
 */
enum class ELogHookType : uint8
{
	Full			= 0x0001,	// Full string match against log line (excluding category and verbosity level)
	Partial			= 0x0002	// Partial string match against log line
};

/** Function to call as the log hook */
using FLogHook = TUniqueFunction<void(const TCHAR* /*Data*/, ELogVerbosity::Type /*Verbosity*/, const class FName& /*Category*/)>;

/** Log hook ID, for removing later if desired */
using FLogHookID = int32;

/**
 * Generic log hook, for calling a hook function upon detecting a specific log entry
 */
class FLogHookManager final : protected FOutputDevice
{
private:
	struct FLogHookEntry
	{
		/** The log hook function to execute */
		FLogHook HookFunc;

		/**	The log string to match on */
		FString LogMatch;

		/** Flags specifying how to match against the log line */
		ELogHookType HookType;

		/** Simple ID for identifying the entry later */
		FLogHookID HookID;
	};

public:
	FLogHookManager() = default;

	virtual ~FLogHookManager() override;

	/**
	 * Adds a log hook, with the specified matching
	 *
	 * @param InHookFunc	The log hook function to call upon match
	 * @param InMatchStr	The string to use for matching against log entries
	 * @param InHookType	The type of matching to use for the match string
	 * @return				Returns a unique id for the newly added log hook entry, which should be used to remove it later
	 */
	FLogHookID AddLogHook(FLogHook&& InHookFunc, FString InMatchStr, ELogHookType InHookType=ELogHookType::Partial);

	/**
	 * Removes a log hook using its unique id
	 *
	 * @param InHookID		The unique id for the log hook entry
	 */
	void RemoveLogHook(FLogHookID InHookID);

private:
#if defined(__clang__)
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Woverloaded-virtual"
#endif
	/**
	 * Detects log entries
	 *
	 * @param Data		The text to write
	 * @param Verbosity	The log level verbosity for the message
	 * @param Category	The log category for the message
	 */
	virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override;
#if defined(__clang__)
#pragma clang diagnostic pop
#endif

	void EnableLogHook();

	void DisableLogHook();

private:
	/** List of active log hooks */
	TArray<FLogHookEntry> LogHooks;

	/** Incrementing log hook ID */
	FLogHookID NextHookID = 0;

	/** Guard bool for preventing recursive calls to Serialize */
	bool bWithinLogSerialize = false;
};

/**
 * A log hook, which watches the log for specified log entries, and ties them into the stack trace manager
 *
 * Most easily used through the LogTrace console command, as documented in UnitTestManager.cpp
 */
class FLogTraceManager final
{
public:
	FLogTraceManager() = default;

	~FLogTraceManager();

	/**
	 * Adds a log line for log trace tracking
	 * NOTE: The LogLine does NOT match the category or verbosity (e.g. LogNet or Warning/Verbose) of logs
	 * NOTE: Partial matches, will output to log when encountered, so matched logs can be identified
	 *
	 * @param LogLine		The line to be tracked
	 * @param TraceFlags	Flags for the type of trace to perform
	 */
	void AddLogTrace(FString LogLine, ELogTraceFlags TraceFlags);

	/**
	 * Removes a log line from trace tracking
	 *
	 * @param LogLine	The log line that was being tracked
	 * @param bDump		Whether or not to dump from the trace manager as well (defaults to true); cleared from trace manager either way
	 */
	void ClearLogTrace(FString LogLine, bool bDump=true);

	/**
	 * Clears all log tracing
	 *
	 * @param bDump		Whether or not to dump all cleared log trace entries
	 */
	void ClearAll(bool bDump=false);

private:
	void ClearHooks();

private:
	/** Active log hook ID's, for later cleanup */
	TArray<FLogHookID> HookIDs;

	/** Active log matching strings and their ID, for tracking and cleanup */
	TMap<FString, FLogHookID> HookMap;
};

/**
 * Log hook which executes console commands upon detecting certain log entries.
 *
 * Used by the LogCommand console command and commandline parameters.
 */
class FLogCommandManager final
{
public:
	FLogCommandManager() = default;

	~FLogCommandManager();

	/**
	 * Adds a new log command
	 *
	 * @param LogLine	The log line to do a partial match on
	 * @param Command	The console command to execute
	 */
	void AddLogCommand(FString LogLine, FString Command);

	/**
	 * Removes a log command based on the log line used for matching
	 *
	 * @param LogLine	The log line used for matching
	 */
	void RemoveByLog(FString LogLine);

	/**
	 * Removes a log command based on the console command it executes
	 */
	void RemoveByCommand(FString Command);

private:
	void ClearHooks();


private:
	struct FLogCommandEntry
	{
		FString		LogLine;
		FString		Command;
		FLogHookID	HookID = INDEX_NONE;
	};

	TArray<FLogCommandEntry> LogCommands;
};

}


/**
 * Flags specifying the type of log trace
 */
enum class ELogTraceFlags : uint16
{
	Full			= 0x0001,	// Full string match log trace (excluding category and verbosity level)
	Partial			= 0x0002,	// Partial string match log trace
	DumpTrace		= 0x0004,	// Dump a stack trace every time there's a match
	Debug			= 0x0008	// Start debugging on match (if debugger attached)
};

ENUM_CLASS_FLAGS(ELogTraceFlags);


/**
 * A log hook, which watches the log for specified log entries, and ties them into the stack trace manager
 *
 * Most easily used through the LogTrace console command, as documented in UnitTestManager.cpp
 */
class UE_DEPRECATED(5.1, "FLogStackTraceManager is deprecated, use FLogTraceManager/GLogTrace now, instead.") FLogStackTraceManager
	: public FOutputDevice
{
public:
	/**
	 * Log trace entry
	 */
	struct FLogTraceEntry
	{
		/**	The log line being to watch for */
		FString LogLine;

		/** Flags specifying the type of log trace */
		ELogTraceFlags TraceFlags;
	};

public:
	FLogStackTraceManager() = default;

	virtual ~FLogStackTraceManager() override;

	/**
	 * Adds a log line for log trace tracking
	 * NOTE: The LogLine does NOT match the category or verbosity (e.g. LogNet or Warning/Verbose) of logs
	 * NOTE: Partial matches, will output to log when encountered, so matched logs can be identified
	 *
	 * @param LogLine	The line to be tracked
	 * @param bPartial	Whether or not to do partial matches for this line (case insensitive, and matches substrings)
	 * @param bDump		Whether or not to dump traces as they are encountered
	 */
	UE_DEPRECATED(5.1, "Use AddLogTrace with ELogTraceFlags now, instead.")
	void AddLogTrace(FString LogLine, bool bPartial=true, bool bDump=true)
	{
		ELogTraceFlags TraceFlags = (bPartial ? ELogTraceFlags::Partial : ELogTraceFlags::Full);

		if (bDump)
		{
			TraceFlags |= ELogTraceFlags::DumpTrace;
		}

		AddLogTrace(LogLine, TraceFlags);
	}

	/**
	 * Adds a log line for log trace tracking
	 * NOTE: The LogLine does NOT match the category or verbosity (e.g. LogNet or Warning/Verbose) of logs
	 * NOTE: Partial matches, will output to log when encountered, so matched logs can be identified
	 *
	 * @param LogLine		The line to be tracked
	 * @param TraceFlags	Flags for the type of trace to perform
	 */
	void AddLogTrace(FString LogLine, ELogTraceFlags TraceFlags);

	/**
	 * Removes a log line from trace tracking
	 *
	 * @param LogLine	The log line that was being tracked
	 * @param bDump		Whether or not to dump from the trace manager as well (defaults to true); cleared from trace manager either way
	 */
	void ClearLogTrace(FString LogLine, bool bDump=true);

	/**
	 * Clears all log tracing
	 *
	 * @param bDump		Whether or not to dump all cleared log trace entries
	 */
	void ClearAll(bool bDump=false);


	// We're hiding UObject::Serialize() by declaring this.  That's OK, but Clang will warn about it.
#if defined(__clang__)
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Woverloaded-virtual"
#endif
	/**
	 * Detects log entries
	 *
	 * @param Data		The text to write
	 * @param Verbosity	The log level verbosity for the message
	 * @param Category	The log category for the message
	 */
	virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override;
#if defined(__clang__)
#pragma clang diagnostic pop
#endif


public:
	/** List of exact log entries to watch for */
	TArray<FLogTraceEntry> ExactMatches;

	/** List of partial log entries to watch for */
	TArray<FLogTraceEntry> PartialMatches;
};


// @todo #JohnBFeatureDebug: Continue implementing (will likely tie into the FLogTraceManager tracing features, so refactor/generalize that)
#if 0
#if !UE_BUILD_SHIPPING
/**
 * A ProcessEvent hook, which watches for the specified function calls, and ties them into the stack trace manager
 */
class FEventStackTraceManager : FProcessEventHookBase
{
public:
	/**
	 * Base constructor
	 */
	FEventStackTraceManager()
		: FProcessEventHookBase()
	{
	}

	virtual void ProcessEventHook(AActor* Actor, UFunction* Function, void* Parameters) override
	{
		// @todo JohnB: Continue
	}
};
#endif
#endif


/**
 * General debug functions
 */
struct NETCODEUNITTEST_API NUTDebug
{
	// @todo #JohnBMerge: Might be useful to find a place for the hexdump functions within the base engine code itself;
	//				they are very useful

	// NOTE: If outputting hex dump info to the unit test log windows, make use of bMonospace, to retain the hex formatting

	/**
	 * Quick conversion of a string to a hex-dumpable byte array
	 *
	 * @param InString		The string to convert
	 * @param OutBytes		The byte array to output to
	 */
	static inline void StringToBytes(const FString& InString, TArray<uint8>& OutBytes)
	{
		TArray<TCHAR, FString::AllocatorType> InStrCharArray = InString.GetCharArray();

		for (int i=0; i<InString.Len(); i++)
		{
			uint8 CharBytes[sizeof(TCHAR)];
			FMemory::Memcpy(&CharBytes[0], &InStrCharArray[i], sizeof(TCHAR));

			for (int CharIdx=0; CharIdx<sizeof(TCHAR); CharIdx++)
			{
				OutBytes.Add(CharBytes[CharIdx]);
			}
		}
	}

	/**
	 * Takes an array of bytes, and generates a hex dump string out of them, optionally including
	 * an ASCII dump and dumping byte offsets also (intended for debugging in the log window)
	 *
	 * @param InBytes		The array of bytes to be dumped
	 * @param bDumpASCII	Whether or not to dump ASCII along with the hex
	 * @param bDumpOffset	Whether or not to dump offset margins for hex rows/columns
	 * @return				Returns the hex dump, including line terminators
	 */
	static FString HexDump(const TArray<uint8>& InBytes, bool bDumpASCII=true, bool bDumpOffset=true);

	/**
	 * Version of the above hex-dump function, which takes a byte pointer and length as input
	 */
	static inline FString HexDump(const uint8* InBytes, const uint32 InBytesLen, bool bDumpASCII=true, bool bDumpOffset=true)
	{
		TArray<uint8> InBytesArray;

		InBytesArray.AddUninitialized(InBytesLen);
		FMemory::Memcpy(InBytesArray.GetData(), InBytes, InBytesLen);

		return HexDump(InBytesArray, bDumpASCII, bDumpOffset);
	}

	/**
	 * Version of the above hex-dump function, which takes a string as input
	 */
	static inline FString HexDump(const FString& InString, bool bDumpASCII=true, bool bDumpOffset=false)
	{
		TArray<uint8> InStrBytes;

		StringToBytes(InString, InStrBytes);

		return HexDump(InStrBytes, bDumpASCII, bDumpOffset);
	}

	/**
	 * Version of the above hex-dump function, which dumps in a format more friendly/readable
	 * in log text files
	 */
	static inline void LogHexDump(const TArray<uint8>& InBytes, bool bDumpASCII=true, bool bDumpOffset=false, FOutputDevice* OutLog=GLog)
	{
		FString HexDumpStr = HexDump(InBytes, bDumpASCII, bDumpOffset);
		TArray<FString> LogLines;

		HexDumpStr.ParseIntoArray(LogLines, LINE_TERMINATOR, false);

		for (int32 i=0; i<LogLines.Num(); i++)
		{
			// NOTE: It's important to pass it in as a parameter, otherwise there is a crash if the line contains '%s'
			OutLog->Logf(TEXT(" %s"), *LogLines[i]);
		}
	}

	/**
	 * Version of the above hex-dump logging function, which takes a byte pointer and length as input
	 */
	static inline void LogHexDump(const uint8* InBytes, const uint32 InBytesLen, bool bDumpASCII=true, bool bDumpOffset=false,
							FOutputDevice* OutLog=GLog)
	{
		TArray<uint8> InBytesArray;

		InBytesArray.AddUninitialized(InBytesLen);
		FMemory::Memcpy(InBytesArray.GetData(), InBytes, InBytesLen);

		LogHexDump(InBytesArray, bDumpASCII, bDumpOffset, OutLog);
	}

	/**
	 * Version of the above, which takes a string as input
	 */
	static inline void LogHexDump(const FString& InString, bool bDumpASCII=true, bool bDumpOffset=false, FOutputDevice* OutLog=GLog)
	{
		TArray<uint8> InStrBytes;

		StringToBytes(InString, InStrBytes);
		LogHexDump(InStrBytes, bDumpASCII, bDumpOffset, OutLog);
	}


	/**
	 * Takes an array of bytes, and generates a bit-based/binary dump string out of them, optionally including
	 * byte offsets also (intended for debugging in the log window)
	 *
	 * @param InBytes		The array of bytes to be dumped
	 * @param bDumpOffset	Whether or not to dump offset margins for bit rows/columns
	 * @param bLSBFirst		Whether or not the Least Significant Bit should be printed first, instead of last
	 * @return				Returns the hex dump, including line terminators
	 */
	static FString BitDump(const TArray<uint8>& InBytes, bool bDumpOffset=true, bool bLSBFirst=false);


	/**
	 * Version of the above bit-dump function, which takes a byte pointer and length as input
	 */
	static inline FString BitDump(const uint8* InBytes, const uint32 InBytesLen, bool bDumpOffset=true, bool bLSBFirst=false)
	{
		TArray<uint8> InBytesArray;

		InBytesArray.AddUninitialized(InBytesLen);
		FMemory::Memcpy(InBytesArray.GetData(), InBytes, InBytesLen);

		return BitDump(InBytesArray, bDumpOffset, bLSBFirst);
	}

	/**
	 * Version of the above bit-dump function, which takes a string as input
	 */
	static inline FString BitDump(const FString& InString, bool bDumpOffset=false, bool bLSBFirst=false)
	{
		TArray<uint8> InStrBytes;

		StringToBytes(InString, InStrBytes);

		return BitDump(InStrBytes, bDumpOffset, bLSBFirst);
	}

	/**
	 * Version of the above bit-dump function, which dumps in a format more friendly/readable
	 * in log text files
	 */
	static inline void LogBitDump(const TArray<uint8>& InBytes, bool bDumpOffset=false, bool bLSBFirst=false, FOutputDevice* OutLog=GLog)
	{
		FString BitDumpStr = BitDump(InBytes, bDumpOffset, bLSBFirst);
		TArray<FString> LogLines;

		BitDumpStr.ParseIntoArray(LogLines, LINE_TERMINATOR, false);

		for (int32 i=0; i<LogLines.Num(); i++)
		{
			// NOTE: It's important to pass it in as a parameter, otherwise there is a crash if the line contains '%s'
			OutLog->Logf(TEXT(" %s"), *LogLines[i]);
		}
	}

	/**
	 * Version of the above bit-dump logging function, which takes a byte pointer and length as input
	 */
	static inline void LogBitDump(const uint8* InBytes, const uint32 InBytesLen, bool bDumpOffset=false, bool bLSBFirst=false,
							FOutputDevice* OutLog=GLog)
	{
		TArray<uint8> InBytesArray;

		InBytesArray.AddUninitialized(InBytesLen);
		FMemory::Memcpy(InBytesArray.GetData(), InBytes, InBytesLen);

		LogBitDump(InBytesArray, bDumpOffset, bLSBFirst, OutLog);
	}

	/**
	 * Version of the above, which takes a string as input
	 */
	static inline void LogBitDump(const FString& InString, bool bDumpOffset=false, bool bLSBFirst=false, FOutputDevice* OutLog=GLog)
	{
		TArray<uint8> InStrBytes;

		StringToBytes(InString, InStrBytes);
		LogBitDump(InStrBytes, bDumpOffset, bLSBFirst, OutLog);
	}
};

