// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Misc/Build.h"
#include "UObject/NameTypes.h"
#include "Delegates/IDelegateInstance.h"


#if STATS
/**
 * Uses the inbuilt UE profiler, to probe the specified profiling events performance,
 * and to detect if the event uses up more than the specified percentage of frame time.
 *
 * NOTE: Only works with SCOPE_CYCLE_COUNTER stats
 * NOTE: Only supports game thread stats, at the moment
 */
class FFrameProfiler
{
public:
	/**
	 * Base constructor
	 *
	 * @param InTargetEvent				The event to be targeted/probed
	 * @param InFramePercentThreshold	The percentage of frame time used by the event, needed to trigger detection
	 */
	FFrameProfiler(FName InTargetEvent, int32 InFramePercentThreshold)
		: TargetEvent(InTargetEvent)
		, FramePercentThreshold(InFramePercentThreshold)
		, bActive(false)
	{
	}

	/**
	 * Base destructor - for ending profiling upon destruct
	 */
	virtual ~FFrameProfiler()
	{
		if (bActive)
		{
			Stop();
		}
	}


	/**
	 * Begins profiling/detection
	 */
	void Start();

	/**
	 * Ends profiling/detection
	 */
	void Stop();

	/**
	 * Profiler hook for notification of new frame data
	 *
	 * @param Frame		The new frame number
	 */
	virtual void OnNewFrame(int64 Frame);


	bool IsActive() const
	{
		return bActive;
	}


public:
	/** The event to be targeted/probed */
	FName TargetEvent;

	/** The percentage of frame time used by the event needed to trigger detection */
	int32 FramePercentThreshold;


protected:
	/** Whether or not profiling is active */
	bool bActive;

	/** Handle to the registered OnNewFrame delegate */
	FDelegateHandle OnNewFrameDelegateHandle;
};
#endif

