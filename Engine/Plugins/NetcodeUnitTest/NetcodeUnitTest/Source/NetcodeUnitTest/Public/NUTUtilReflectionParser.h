// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ExpressionParser.h"

class FVMReflection;

/**
 * FVMReflectionParser - A string interface for FVMReflection, which performs reflection on the UE virtual machine.
 *
 * This allows access to reflection through console commands, using a lexer which implements C++ style syntax for parsing.
 *
 * This aims to provide easy access to everything in the UE VM - like a supercharged version of the 'get/set' commands,
 * able to access and step-through any variable/array/struct type and call any function with any parameter types, with no limitations
 * (all of which FVMReflection is capable of on its own, but lacking a string interface).
 */


/**
 * Provides a context for evaluating expressions, where the FVMReflection object is initialized/passed-on
 */
struct FReflEvaluationContext
{
	/** The reflection handler containing the current state of reflection */
	TSharedPtr<FVMReflection> Refl;

	/**
	 * Default constructor
	 */
	FReflEvaluationContext()
		: Refl(nullptr)
	{
	}
};

/**
 * There is a problem with the parser API, where the Context you pass into the Evaluate function is marked: const ContextType* Context
 *
 * This means you can not modify Context (which I need to do, to pass state information within the parser), but there is a loophole,
 * where you are able to modify anything Context points to, so I use this struct to wrap the REAL context (FReflEvaluationContext),
 * as a pointer in a dud struct.
 */
struct FContextPointer
{
	FReflEvaluationContext* Context;

	FContextPointer(FReflEvaluationContext* InContext)
		: Context(InContext)
	{
	}
};

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
struct FTestResultPointer
{
	struct FTestResult* Result;

	FTestResultPointer(FTestResult* InResult)
		: Result(InResult)
	{
	}
};
#endif


/**
 * FVMReflectionParser
 */
class FVMReflectionParser
{
public:
	/**
	 * Base constructor
	 */
	FVMReflectionParser();


	/**
	 * Evaluate the given reflection expression, into an FVMReflection instance
	 *
	 * @param InExpression	The expression to evaluate
	 * @param InTargetObj	Optionally, the target object to perform reflection on
	 * @return				Returns either a shared pointer to an FVMReflection instance, or an error
	 */
	TValueOrError<TSharedPtr<FVMReflection>, FExpressionError> Evaluate(const TCHAR* InExpression, UObject* InTargetObj=nullptr) const;

	/**
	 * As above, except converts whatever the final reflection state points to, into a human readable string
	 *
	 * @param InExpression	The expression to evaluate
	 * @param InTargetObj	Optionally, the target object to perform reflection on
	 * @return				Returns either a string representing the final reflection state, or an error
	 */
	TValueOrError<FString, FExpressionError> EvaluateString(const TCHAR* InExpression, UObject* InTargetObj=nullptr) const;

protected:
	/**
	 * Attempts to parse an identifier token (variable/function name) from the stream
	 *
	 * @param InStream		The stream to attempt token parsing on
	 * @param Accumulate	Optionally specify an existing string token, to append the parsed token to
	 * @return				If successful, returns the parsed string token, if false, returns nothing and IsSet will be false.
	 */
	static TOptional<FStringToken> ParseIdentifier(const FTokenStream& InStream, FStringToken* Accumulate=nullptr);

	/**
	 * Attempts to parse an array subscript operator, Array[Num], from the stream
	 *
	 * @param InStream		The stream to attempt token parsing on
	 * @param Accumulate	Optionally specify an existing string token, to append the parsed token to
	 * @return				If successful, returns the parsed string token, if false, returns nothing and IsSet will be false.
	 */
	static TOptional<FStringToken> ParseArraySubscript(const FTokenStream& InStream, FStringToken* Accumulate=nullptr);


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
public:
	/**
	 * Initializes automated testing parser parameters.
	 */
	void TestConstruct();

	/**
	 * Evaluates a test expression
	 *
	 * @param InExpression	The expression to evaluate
	 * @return				Returns the result of the test
	 */
	TValueOrError<FString, FExpressionError> TestEvaluate(const TCHAR* InExpression) const;
#endif

protected:
	/** A dictionary used for defining how tokens are lexed */
	FTokenDefinitions TokenDefinitions;

	/** Used to define the lexical grammar for how an expression should be parsed, e.g. defining operators/preoperators etc. */
	FExpressionGrammar Grammar;

	/** Table for mapping operator definitions, to a function which should evaluate the operator */
	TOperatorJumpTable<FContextPointer> OpJumpTable;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Table for test operators */
	TOperatorJumpTable<FTestResultPointer> TestOpJumpTable;
#endif
};
