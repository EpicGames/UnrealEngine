// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Engine/Channel.h"
#include "NetcodeUnitTest.h"
#include "Engine/NetworkDelegates.h"
#include "Engine/World.h"
#include "Sockets.h"
#include "UObject/GCObject.h"

#include "Net/UnitTestPackageMap.h"


// Forward declarations
class APlayerController;
class FInBunch;
class FInternetAddr;
class FOutBunch;
class FRepLayoutCmd;
class UNetConnection;
class UNetDriver;
struct FUniqueNetIdRepl;

class UMinimalClient;
class UUnitTestChannel;
class UUnitTestNetDriver;
class UClientUnitTest;


// Delegates

/**
 * Delegate used to control/block receiving of RPC's
 *
 * @param Actor			The actor the RPC will be called on
 * @param Function		The RPC to call
 * @param Parameters	The parameters data blob
 * @param bBlockRPC		Whether or not to block execution of the RPC
*/
DECLARE_DELEGATE_FourParams(FOnProcessNetEvent, AActor* /*Actor*/, UFunction* /*Function*/, void* /*Parameters*/, bool& /*bBlockRPC*/);

/**
 * Delegate used to control hooked serialization of RPC parameters
 *
 * @param Ar			The archive being serialized to
 * @param Map			The PackageMap for the net connection
 * @param Data			The raw property data
 * @param MetaData		The meta data for the property
 */
DECLARE_DELEGATE_RetVal_FourParams(bool, FOnNetSerializeItem, FArchive& /*Ar*/, UPackageMap* /*Map*/, void* /*Data*/,
									TArray<uint8>* /*MetaData*/);


/**
 * Class for encapsulating ProcessEvent and general RPC hooks, implemented globally for each UWorld
 * NOTE: Presently, all RPC hooks tied to UWorld's only hook client RPC's - global RPC hooks, hook both client and server RPC's
 */
class FProcessEventHook
{
public:
	/**
	 * Get a reference to the net event hook singular
	 */
	static FProcessEventHook& Get()
	{
		static FProcessEventHook* HookSingular = nullptr;

		if (HookSingular == nullptr)
		{
			HookSingular = new FProcessEventHook();
		}

		return *HookSingular;
	}

public:
	/**
	 * Default constructor
	 */
	FProcessEventHook()
		: NetEventHooks()
		, EventHooks()
		, GlobalNetEventHooks()
		, GlobalEventHooks()
	{
	}

	/**
	 * Adds an RPC hook for the specified UWorld.
	 *
	 * @param InWorld	The UWorld whose RPC's should be hooked
	 * @param InHook	The delegate for handling the hooked RPC's
	 */
	void AddRPCHook(UWorld* InWorld, FOnProcessNetEvent InHook);

	/**
	 * Removes the RPC hook for the specified UWorld.
	 *
	 * @param InWorld	The UWorld whose RPC hook should be cleared.
	 */
	void RemoveRPCHook(UWorld* InWorld);

	/**
	 * Adds a non-RPC hook for the specified UWorld.
	 *
	 * @param InWorld	The UWorld whose RPC's should be hooked
	 * @param InHook	The delegate for handling the hooked events
	 */
	void AddEventHook(UWorld* InWorld, FOnProcessNetEvent InHook);

	/**
	 * Removes the hook for the specified UWorld.
	 *
	 * @param InWorld	The UWorld whose RPC hook should be cleared.
	 */
	void RemoveEventHook(UWorld* InWorld);

	/**
	 * Adds a global RPC hook.
	 *
	 * @param InHook	The delegate for handling the hooked RPC's
	 * @return			Returns a delegate handle, used for removing the hook later
	 */
	FDelegateHandle AddGlobalRPCHook(FOnProcessNetEvent InHook);

	/**
	 * Removes the specified global RPC hook
	 *
	 * @param InHandle	The handle to the global RPC hook to remove
	 */
	void RemoveGlobalRPCHook(FDelegateHandle InHandle);

	/**
	 * Adds a global non-RPC hook.
	 *
	 * @param InHook	The delegate for handling the hooked events
	 * @return			Returns a delegate handle, used for removing the hook later
	 */
	FDelegateHandle AddGlobalEventHook(FOnProcessNetEvent InHook);

	/**
	 * Removes the specified global non-RPC hook
	 *
	 * @param InHandle	The handle to the global non-RPC hook to remove
	 */
	void RemoveGlobalEventHook(FDelegateHandle InHandle);


private:
	/**
	 * Checks performed prior to adding any hooks
	 */
	void PreAddHook();

	/**
	 * Checks performed after removing any hooks
	 */
	void PostRemoveHook();

	/**
	 * Base hook for AActor::ProcessEventDelegate - responsible for filtering based on RPC events and Actor UWorld
	 */
	bool HandleProcessEvent(AActor* Actor, UFunction* Function, void* Parameters);

private:
	/** The global list of RPC hooks, and the UWorld they are associated with */
	TMap<UWorld*, FOnProcessNetEvent> NetEventHooks;

	/** The global list of ProcessEvent hooks, and the UWorld they are associated with */
	TMap<UWorld*, FOnProcessNetEvent> EventHooks;

	/** The global list of RPC hooks, which aren't associated with a specific UWorld */
	TArray<FOnProcessNetEvent> GlobalNetEventHooks;

	/** The global list of ProcessEvent hooks, which aren't associated with a specific UWorld */
	TArray<FOnProcessNetEvent> GlobalEventHooks;
};


/**
 * A delegate network notify class, to allow for easy inline-hooking.
 * 
 * NOTE: This will memory leak upon level change and re-hooking (if used as a hook),
 * because there is no consistent way to handle deleting it.
 */
class FNetworkNotifyHook : public FNetworkNotify
{
public:
	/**
	 * Base constructor
	 */
	FNetworkNotifyHook()
		: FNetworkNotify()
		, NotifyAcceptingConnectionDelegate()
		, NotifyAcceptedConnectionDelegate()
		, NotifyAcceptingChannelDelegate()
		, NotifyControlMessageDelegate()
		, HookedNotify(NULL)
	{
	}

	/**
	 * Constructor which hooks an existing network notify
	 */
	FNetworkNotifyHook(FNetworkNotify* InHookNotify)
		: FNetworkNotify()
		, HookedNotify(InHookNotify)
	{
	}

	/**
	 * Virtual destructor
	 */
	virtual ~FNetworkNotifyHook()
	{
	}


protected:
	/**
	 * Old/original notifications
	 */

	virtual EAcceptConnection::Type NotifyAcceptingConnection() override;

	virtual void NotifyAcceptedConnection(UNetConnection* Connection) override;

	virtual bool NotifyAcceptingChannel(UChannel* Channel) override;

	virtual void NotifyControlMessage(UNetConnection* Connection, uint8 MessageType, FInBunch& Bunch) override;


	DECLARE_DELEGATE_RetVal(EAcceptConnection::Type, FNotifyAcceptingConnectionDelegate);
	DECLARE_DELEGATE_OneParam(FNotifyAcceptedConnectionDelegate, UNetConnection* /*Connection*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FNotifyAcceptingChannelDelegate, UChannel* /*Channel*/);
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FNotifyControlMessageDelegate, UNetConnection* /*Connection*/, uint8 /*MessageType*/,
										FInBunch& /*Bunch*/);

public:
	FNotifyAcceptingConnectionDelegate	NotifyAcceptingConnectionDelegate;
	FNotifyAcceptedConnectionDelegate	NotifyAcceptedConnectionDelegate;
	FNotifyAcceptingChannelDelegate		NotifyAcceptingChannelDelegate;
	FNotifyControlMessageDelegate		NotifyControlMessageDelegate;

	/** If this is hooking an existing network notify, save a reference */
	FNetworkNotify* HookedNotify;
};


/**
 * Class for hooking world tick events, and setting globals for log hooking
 */
class FWorldTickHook
{
private:
	// Internal GC hook, due to 'Init' function name conflict
	class FInternalGCObject : public FGCObject
	{
	public:
		FInternalGCObject() = delete;

		FInternalGCObject(FWorldTickHook* InOwner)
			: Owner(InOwner)
		{
		}

		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;

	private:
		FWorldTickHook* Owner;
	};

public:
	FWorldTickHook(UWorld* InWorld)
		: AttachedWorld(InWorld)
		, InternalGC(this)
	{
	}

	void Init();

	void Cleanup();

private:
	void TickDispatch(float DeltaTime);

	void PostTickFlush();

public:
	/** The world this is attached to */
	TObjectPtr<UWorld> AttachedWorld;

private:
	/** Handle for Tick dispatch delegate */
	FDelegateHandle TickDispatchDelegateHandle;

	/** Handle for PostTick dispatch delegate */
	FDelegateHandle PostTickFlushDelegateHandle;

	/** Internal GC */
	FInternalGCObject InternalGC;
};


// @todo #JohnB: Refactor to be based on minimal client
/**
 * Hooks netcode object serialization, in order to replace replication of a specific object, with another specified object,
 * for the lifetime of the scoped instance
 */
class NETCODEUNITTEST_API FScopedNetObjectReplace
{
public:
	FScopedNetObjectReplace(UClientUnitTest* InUnitTest, UObject* InObjToReplace, UObject* InObjReplacement);

	~FScopedNetObjectReplace();


private:
	UClientUnitTest* UnitTest;

	UObject* ObjToReplace;
};

/**
 * Hooks netcode name serialization, in order to replace replication of a specific name, with another specified name,
 * for the lifetime of the scoped instance
 */
class NETCODEUNITTEST_API FScopedNetNameReplace
{
public:
	FScopedNetNameReplace(UMinimalClient* InMinClient, const FOnSerializeName::FDelegate& InDelegate);

	// @todo #JohnB: This variation is not tested
	FScopedNetNameReplace(UMinimalClient* InMinClient, FName InNameToReplace, FName InNameReplacement);

	~FScopedNetNameReplace();

private:
	UMinimalClient* MinClient;

	FDelegateHandle Handle;
};

/**
 * Hooks the netcode serialization of a specific RPC parameter, in order to override the raw serialized param data with something else
 */
class NETCODEUNITTEST_API FScopedRPCParamReplace
{
public:
	FScopedRPCParamReplace(UMinimalClient* InMinClient, UFunction* InTargetRPC, FString InParamName,
							const FOnNetSerializeItem& InSerializeHook);

	~FScopedRPCParamReplace();

private:
	FRepLayoutCmd*		ParamRepCmd;

	FProperty*			OriginalParam;
};

/**
 * Implements the actual hook for the above class
 */
class FNetPropertyHook : public FProperty
{
	DECLARE_FIELD_API(FNetPropertyHook, FProperty, CASTCLASS_None, NETCODEUNITTEST_API)

private:
	virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData /*=nullptr*/) const override;

public:

	FNetPropertyHook(FFieldVariant InOwner, const FName& InName)
		: FProperty(InOwner, InName)
	{}

	FOnNetSerializeItem SerializeHook;
};


/**
 * Netcode based utility functions
 */
struct NUTNet
{
	/**
	 * Handles setting up the client beacon once it is replicated, so that it can properly send RPC's
	 * (normally the serverside client beacon links up with the pre-existing beacon on the clientside,
	 * but with unit tests there is no pre-existing clientside beacon)
	 *
	 * @param InBeacon		The beacon that should be setup
	 * @param InConnection	The connection associated with the beacon
	 */
	static NETCODEUNITTEST_API void HandleBeaconReplicate(AActor* InBeacon, UNetConnection* InConnection);


	/**
	 * Creates a barebones/minimal UWorld, for setting up minimal client connections,
	 * and as a container for objects in the unit test commandlet
	 *
	 * @return	Returns the created UWorld object
	 */
	static NETCODEUNITTEST_API UWorld* CreateUnitTestWorld(bool bHookTick=true);

	/**
	 * Marks the specified unit test world for cleanup
	 *
	 * @param CleanupWorld	The unit test world to be marked for cleanup
	 * @param bImmediate	If true, all unit test worlds pending cleanup, are immediately cleaned up
	 */
	static void MarkUnitTestWorldForCleanup(UWorld* CleanupWorld, bool bImmediate=false);

	/**
	 * Cleans up unit test worlds queued for cleanup
	 */
	static void CleanupUnitTestWorlds();

	/**
	 * Returns true, if the specified world is a unit test world
	 */
	static bool IsUnitTestWorld(UWorld* InWorld);
};
