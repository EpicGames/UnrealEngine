// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Logging/LogMacros.h"


// Forward declarations
class UUnitTestManager;



/**
 * Globals/externs
 */

/** Holds a reference to the object in charge of managing unit tests */
extern UUnitTestManager* GUnitTestManager;

/** Whether not an actor channel, is in the process of initializing the remote actor */
extern bool GIsInitializingActorChan;


/**
 * Declarations
 */

NETCODEUNITTEST_API DECLARE_LOG_CATEGORY_EXTERN(LogUnitTest, Log, All);

// Hack to allow log entries to print without the category (specify log type of 'none')
DECLARE_LOG_CATEGORY_EXTERN(NetCodeTestNone, Log, All);


/**
 * Defines
 */

/**
 * The IPC pipe used for resuming a suspended server
 * NOTE: You must append the process ID of the server to this string
 */
#define NUT_SUSPEND_PIPE TEXT("\\\\.\\Pipe\\NetcodeUnitTest_SuspendResume")


/**
 * Macro defines
 */

// Actual asserts (not optimized out, like almost all other engine assert macros)
// @todo #JohnBLowPri: Try to get this to show up a message box, or some other notification, rather than just exiting immediately
#define UNIT_ASSERT(Condition) \
	if (!(Condition)) \
	{ \
		UE_LOG(LogUnitTest, Error, TEXT(UE_STRINGIZE(__FILE__)) TEXT(": Line: ") TEXT(UE_STRINGIZE(__LINE__)) \
				TEXT(":")); \
		UE_LOG(LogUnitTest, Error, TEXT("Condition '(") TEXT(UE_STRINGIZE(Condition)) TEXT(")' failed")); \
		check(false); /* Try to get a meaningful stack trace. */ \
		FPlatformMisc::RequestExit(true); \
		CA_ASSUME(false); \
	}


