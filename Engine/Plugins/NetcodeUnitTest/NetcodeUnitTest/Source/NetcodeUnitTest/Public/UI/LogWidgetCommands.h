// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Styling/CoreStyle.h"
#include "Framework/Commands/Commands.h"

/**
 * Defines a set of commands used by the unit test log window
 */
class FLogWidgetCommands : public TCommands<FLogWidgetCommands>
{
public:
	FLogWidgetCommands()
		: TCommands<FLogWidgetCommands>("NUTLogWidget", FText::FromString(TEXT("Unit Test Log")), NAME_None,
											FCoreStyle::Get().GetStyleSetName())
		, CopyLogLines(NULL)
		, FindLogText(NULL)
	{
	}

	/**
	 * Register commands
	 */
	void RegisterCommands() override;


public:
	/** Command for copying the currently selected log line(s) */
	TSharedPtr<FUICommandInfo>	CopyLogLines;

	/** Command for finding text within the current log window tab */
	TSharedPtr<FUICommandInfo>	FindLogText;
};

