// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "NetcodeUnitTest.h"
#include "UnitLogging.h"


// Forward declarations
class SLogWindow;
class SWindow;

/**
 * @todo #JohnBFeatureUI: Slate log windows
 *
 * What is needed:
 *	- Ability to enter console commands into log window (what context should they execute in? game client, exploit, server?)
 *		- Context is a good question - maybe add prefixes 'local', 'server' and 'exploit', for specifying context
 *			(and add some way of providing a note on that, like a tooltip or something)
 *	- Should there be actual log files output, for each log window?
 *		- Longfinger it, as a 'todo'
 *
 *
 * Problems:
 *	- Is it better to have lots of individual desktop-level log windows, or to encapsulate them in one big slate window?
 *		- Encapsulation is probably better, as cluttering the desktop is a pain if you want to change focus
 *			- It is more work though, so put it off until necessary - but definitely make it a 'todo',
 *				and have a configurable option, to switch between 'one window' and 'many windows'
 *
 * Notes:
 *	- It is important, that you don't omit logs from the main client log - because if you do, debugging issues would involve
 *		potentially having to trace messages across two logs, which is extremely bad
 *	- Should the local client log hook, perhaps be added as an official log hook in the exploit? (since it would be useful,
 *		for verification)
 *		- Yes, give this a try
 *		- UPDATE: This means, adding a verification function, much like you have done with the server version of this function;
 *					that could all do with cleanup anyway
 */

// Forward declarations
class SLogWindow;


/**
 * Struct representing a grid location for a log window
 */
struct FLogGridEntry
{
	FLogGridEntry()
		: LogWindow(NULL)
		, Top(0.f)
		, Bottom(0.f)
		, Left(0.f)
		, Right(0.f)
	{
	}

	/* A reference to the log window inhabiting this grid space (if any) */
	TSharedPtr<SLogWindow> LogWindow;

	/** The dimensions of the grid entry */
	float Top;
	float Bottom;
	float Left;
	float Right;
};


/**
 * Basic window manager, for handling tiling of unit test log windows.
 *
 * How this works:
 * Upon initialization, takes the average dimensions of a log window, and of the desktop working area,
 * and maps out a grid for fitting the log windows onto the desktop - nothing more complicated than that
 */
class FLogWindowManager
{
public:
	FLogWindowManager()
		: bInitialized(false)
		, GridSpaces()
		, LastLogWindowPos(-1)
		, OverflowWindows()
		, LogWidth(0)
		, LogHeight(0)
	{
	}

	/**
	 * Base destructor
	 */
	~FLogWindowManager();

	/**
	 * Initializes the log window manager
	 *
	 * @param InLogWidth	The width that log windows will have
	 * @param InLogHeight	The height that log windows will have
	 */
	void Initialize(int InLogWidth, int InLogHeight);

	/**
	 * Creates a new log window, and returns a reference
	 *
	 * @param Title				The title to give the long window
	 * @param ExpectedFilters	The type of log output that is expected - used for setting up tabs
	 * @param bStatusWindow		Whether or not the window being created is the status window
	 * @return					A reference to the created log window
	 */
	TSharedPtr<SLogWindow> CreateLogWindow(FString Title=TEXT(""), ELogType ExpectedFilters=ELogType::None, bool bStatusWindow=false);


	/**
	 * Monitor for closed log windows
	 *
	 * @param ClosedWindow	The window that was closed
	 */
	void OnWindowClosed(const TSharedRef<SWindow>& ClosedWindow);


protected:
	/**
	 * Finds the next free grid position for the log window
	 * NOTE: Instead of searching for the first empty space, tries to create log windows in a row pattern based upon the grid
	 */
	int FindFreeGridPos();


protected:
	/** Whether or not the log window manager has been initialized */
	bool								bInitialized;


	/** An array of currently open log windows */
	TArray<FLogGridEntry>				GridSpaces;

	/** The last grid entry where a log window was placed */
	int									LastLogWindowPos;

	/** A list of windows that have overflowed the grid (these start minimized, waiting for a space in the grid) */
	TArray<TSharedPtr<SLogWindow>>		OverflowWindows;


	/** The dimensions log windows should have */
	int									LogWidth;
	int									LogHeight;


	/** Handles to registered OnWindowClosed delegates for particular windows */
	TMap<SLogWindow*, FDelegateHandle> OnWindowClosedDelegateHandles;
};


