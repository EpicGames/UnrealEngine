// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "UObject/NoExportTypes.h"
#include "Internationalization/Text.h"
#include "Containers/UnrealString.h"

class SWindow;

// Largely a copy of UnrealEd Dialogs.h

// Warning: Currently all 'window close' events for the dialog are treated as user input, so be careful if closing within code
//			(best to call SetOnWindowClosed(FOnWindowClosed()) prior to calling)

/**
 * Result callback delegate, for below function
 *
 * @param DialogWindow	The dialog returning a response
 * @param Result		The result/response
 * @param bNoResult		Whether or not the result is invalid, e.g. the dialog was closed instead of a button clicked
 */
DECLARE_DELEGATE_ThreeParams(FOnLogDialogResult, const TSharedRef<SWindow>& /* DialogWindow */, EAppReturnType::Type /* Result */,
								bool /* bNoResult */);

/**
 * Opens a non-modal/non-blocking message box, which returns its result through a delegate/callback,
 * using a reference to the created window, to identify which dialog has returned a result (in case there are multiple dialog windows)
 *
 * @param InMessageType		The type of message box to display (e.g. 'ok', or 'yes'/'no' etc.)
 * @param InMessage			The message to display in the message box
 * @param InTitle			The title to display for the message box
 * @param ResultCallback	Optionally, specify a delegate/callback instance, where results should be returned
 * @return					Returns the dialog window reference, which the calling code should store, to identify which dialog returned
 */
TSharedRef<SWindow> OpenLogDialog_NonModal(EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle,
											FOnLogDialogResult ResultCallback=NULL);

FORCEINLINE TSharedRef<SWindow> OpenLogDialog_NonModal(EAppMsgType::Type InMessageType, const FString& InMessage,
											const FString& InTitle, FOnLogDialogResult ResultCallback=NULL)
{
	return OpenLogDialog_NonModal(InMessageType, FText::FromString(InMessage), FText::FromString(InTitle), ResultCallback);
}

