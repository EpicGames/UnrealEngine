// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Misc/CommandLine.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Framework/Docking/TabManager.h"

#include "NetcodeUnitTest.h"
#include "UnitLogging.h"


// Forward declarations
class FUICommandList;
class SEditableTextBox;
enum class ESuspendState : uint8;

// Delegate definitions

/**
 * Delegate for neatly passing on suspend/resume button click
 */
DECLARE_DELEGATE(FOnSuspendClicked);

/**
 * Delegate for passing on 'developer' mode checkbox clicks
 *
 * @param bDeveloperMode	Whether developer mode has been enabled/disabled
 */
DECLARE_DELEGATE_OneParam(FOnDeveloperClicked, bool /*bDeveloperMode*/)

/**
 * Delegate for passing on console commands from the UI, to the unit test
 *
 * @param CommandContext	The context (local/server/client?) for the console command
 * @param Command			The command to be executed
 * @return					Whether or not the command was handled - determines if the UI wipes the command textbox
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnConsoleCommand, FString /*CommandContext*/, FString /*Command*/);


// Structs

/**
 * Struct representing a log line
 */
struct FLogLine
{
	/** The unit test log type */
	ELogType			LogType;

	/** The raw log line string */
	TSharedRef<FString>	LogLine;

	/** The color of the log line */
	FSlateColor			LogColor;

	/** The text within the log line, to highlight (from searches) */
	FString				LogHighlight;


	FLogLine(ELogType InLogType, TSharedRef<FString> InLogLine, FSlateColor InLogColor)
		: LogType(InLogType)
		, LogLine(InLogLine)
		, LogColor(InLogColor)
		, LogHighlight()
	{
	}
};

/**
 * Struct containing tab-specific information
 */
struct FLogTabInfo
{
	/** The label to apply to the tab */
	FString		Label;

	/** The tooltip/description to apply to the tab */
	FString		ToolTip;

	/** The log line filter the tab should use */
	ELogType	Filter;

	/** When a logged line requests focus, this determines the priority for selecting what tab to focus (lower = higher priority) */
	uint8		Priority;

	/** Whether or not the tab is presently open (some tabs, such as the 'debug' tab, start off closed if not explicitly enabled) */
	bool		bTabOpen;


	/** The list of log lines for this particular tab */
	TArray<TSharedRef<FLogLine>> TabLogLines;

	/** Reference to the log list view for the tab */
	TSharedPtr<SListView<TSharedRef<FLogLine>>> LogListView;


	/** The list of widgets for the 'Find' bar */
	TArray<TSharedPtr<SWidget>> FindWidgets;

	/** The text box for the 'Find' bar */
	TSharedPtr<SEditableTextBox> FindBox;

	/** The label that's displayed when there's an error searching, or searching past the start/end of the list */
	TSharedPtr<SBorder> FindErrorLabel;

	/** The text block for the above label */
	TSharedPtr<STextBlock> FindErrorText;

	/** Caches the text of the last search */
	FString LastFind;

	/** Whether or not the last search direction was upwards */
	bool bLastFindWasUp;

	/** Whether or not to highlight text, from searches */
	bool bHighlightFind;


	/** Automatically generated name given to the tab */
	FName TabIdName;


	/** Stores a reference to the created tab (a weak reference, so it does not block destruction, or require cleanup) */
	// @todo JohnB: Convert to shared pointer, once/if you implement closing of tabs - the closing code, should then unset this value
	TWeakPtr<SDockTab> TabWidget;


	// @todo #JohnBFeatureUI: Add an extended filter eventually (perhaps an FString for simple String.Contains filter?)


	FLogTabInfo(FString InLabel, FString InToolTip, ELogType InFilter=ELogType::All, uint8 InPriority=255, bool bInTabOpen=true)
		: Label(InLabel)
		, ToolTip(InToolTip)
		, Filter(InFilter)
		, Priority(InPriority)
		, bTabOpen(bInTabOpen)
		, TabLogLines()
		, LogListView()
		, FindWidgets()
		, FindBox()
		, FindErrorLabel()
		, FindErrorText()
		, LastFind()
		, bLastFindWasUp(false)
		, bHighlightFind(true)
		, TabWidget()
	{
		static uint8 TabNameCounter = 0;

		// @todo JohnB: WARNING: Not compatible with restoring from saved layouts (though currently doesn't matter)
		TabIdName = FName(*FString::Printf(TEXT("Tab%i"), TabNameCounter++));
	}
};


class SLogWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLogWidget)
		: _bStatusWidget(false)
		, _ExpectedFilters(ELogType::None)
	{
	}
		/** Whether or not this is the unit-test-status window */
		SLATE_ARGUMENT(bool, bStatusWidget)

		/** Input argument, outlining the log filters expected for this log window - used to decide which tabs should be added */
		SLATE_ARGUMENT(ELogType, ExpectedFilters)
	SLATE_END_ARGS()

	SLogWidget()
		: SCompoundWidget()
		, LogLines()
		, LogTabs()
		, SuspendButtonText()
		, ConsoleContextList()
		, DefaultConsoleContext(TEXT(""))
		, ConsoleComboBox()
		, ConsoleTextBox()
		, LogWidgetCommands(NULL)
		, bAutoScroll(true)
		, OnSuspendClicked()
		, OnDeveloperClicked()
		, OnConsoleCommand()
	{
		bAutoClose = !FParse::Param(FCommandLine::Get(), TEXT("UnitTestNoAutoClose"));
	}

	/**
	 * Widget constructor
	 */
	void Construct(const FArguments& Args);

protected:
	/**
	 * Initializes the LogTabInfo array and tab formatting, and returns the log tab formatting layout
	 *
	 * @param Args	The widget constructions args
	 * @return		The layout to be used for initializing the tab widgets
	 */
	TSharedRef<FTabManager::FLayout> InitializeTabLayout(const FArguments& Args);

	/**
	 * Spawns an individual tab for the log list view
	 */
	TSharedRef<SDockTab> SpawnLogTab(const FSpawnTabArgs& InSpawnTabArgs);

	/**
	 * Returns a reference to the LogTabs entry, for the currently active tab
	 *
	 * @return		The LogTabs entry, for the currently active tab
	 */
	TSharedPtr<FLogTabInfo> GetActiveTabInfo() const;

public:
	/**
	 * Adds a log line to the log window
	 *
	 * @param InLogType			The type of log output this is (for filtering)
	 * @param LogLine			The log line
	 * @param LogColor			The colour to apply to the log line
	 * @param bTakeTabFocus		If this log line isn't displayed on the currently focused tab, switch focus to a tab that does show it
	 */
	void AddLine(ELogType InLogType, TSharedRef<FString> LogLine, FSlateColor LogColor=FSlateColor::UseForeground(),
					bool bTakeTabFocus=false);


	/**
	 * Notification from the unit test (through a delegate), that the server suspend state has changed (for the button text)
	 *
	 * @param InSuspendState	The new server suspend state
	 */
	void OnSuspendStateChanged(ESuspendState InSuspendState);


protected:
	/**
	 * Called after a key is pressed when this widget has keyboard focus
	 *
	 * @param	MyGeometry The Geometry of the widget receiving the event
	 * @param	InKeyboardEvent  Keyboard event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyboardEvent) override;

	/**
	 * Called when the 'Copy' context button is clicked
	 */
	void OnCopy();

	/**
	 * Used to determine whether 'Copy' is currently a valid action
	 *
	 * @return	Whether or not a copy can be performed right now
	 */
	bool CanCopy() const;

	/**
	 * Called when the 'Find' context button is clicked
	 */
	void OnFind();

	/**
	 * Used to determine whether 'Find' is currently a valid action
	 *
	 * @return	Whether or not find can be performed right now
	 */
	bool CanFind() const;


	/**
	 * Scrolls the specified tab to the last entry
	 *
	 * @param InTab		The tab which should scroll to the end
	 */
	void ScrollToEnd(TSharedRef<FLogTabInfo> InTab);

	/**
	 * Scrolls to the specified text, in the specified tab (starting from the currently selected line)
	 *
	 * @param InTab			The tab that the search is being performed in
	 * @param FindText		The text to find
	 * @param bSearchUp		Whether or not to search up instead of down
	 */
	void ScrollToText(TSharedRef<FLogTabInfo> InTab, FString FindText, bool bSearchUp=false);

	/**
	 * Whether or not auto-scrolling is presently possible for the specified tab
	 * NOTE: Does NOT determine whether or not it is presently enabled though
	 *
	 * @param InTab		The tab to check
	 * @return			Whether or not auto-scrolling is possible
	 */
	bool CanAutoScroll(TSharedPtr<FLogTabInfo> InTab);

	/**
	 * Updates the find text highlighting for a tab
	 *
	 * @param InTab				The tab to update
	 * @param bHighlight		Whether or not highlighting has been enabled (NOTE: This often differs from the tabs bHighlightFind value)
	 * @param HighlightText		The text to highlight, if any
	 */
	void UpdateFindHighlight(TSharedPtr<FLogTabInfo> InTab, bool bHighlight, FString HighlightText=TEXT(""));


	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;


protected:
	/** The log window tab manager */
	TSharedPtr<FTabManager> LogTabManager;

	/** The full list of all accumulated log lines - stored centrally/separately from tabs */
	// @todo JohnB: Don't get rid of this - will need it later on, to initialize tabs added on-the-fly
	TArray<TSharedRef<FLogLine>> LogLines;

	/** Holds information necessary for initializing and handling the state of each tab */
	TArray<TSharedRef<FLogTabInfo>> LogTabs;


public:
	/** Reference to the suspend buttons text, so the text can be modified */
	TSharedPtr<STextBlock> SuspendButtonText;


	/** The list of available console command contexts (for the combo box) */
	TArray<TSharedPtr<FString>> ConsoleContextList;

	/** The default console command context (for the combo box) */
	FString DefaultConsoleContext;

protected:
	/** The combo box for selecting the console command context */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ConsoleComboBox;

	/** The console command text box */
	TSharedPtr<SEditableTextBox> ConsoleTextBox;


	/** Log widget context-menu/shortcut-key commands */
	TSharedPtr<FUICommandList> LogWidgetCommands;


	/** Whether or not auto-scrolling is currently enabled */
	bool bAutoScroll;

public:
	/** Whether or not auto-closing the current window is enabled */
	bool bAutoClose;


	/** Notification for suspend/resume button click */
	FOnSuspendClicked OnSuspendClicked;

	/** Notification for developer checkbox click */
	FOnDeveloperClicked OnDeveloperClicked;

	/** Delegate for handling UI-triggered console commands */
	FOnConsoleCommand OnConsoleCommand;
};
