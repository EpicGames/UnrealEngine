// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWindow.h"

#include "NetcodeUnitTest.h"
#include "UnitLogging.h"


// Forward declarations
class SLogWidget;

/**
 * Setup a multicast version of the standard OnWindowClosed delegate
 *
 * @param ClosedWindow	The window that has closed
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FMultiOnWindowClosed, const TSharedRef<SWindow>& /*ClosedWindow*/);


class SLogWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SLogWindow)
		: _bStatusWindow(false)
		, _ExpectedFilters(ELogType::None)
	{
	}
		SLATE_ARGUMENT(bool, bStatusWindow)
		SLATE_ARGUMENT(ELogType, ExpectedFilters)
	SLATE_END_ARGS()

	SLogWindow()
		: LogWidget(NULL)
		, MultiOnWindowClosed()
		, bHasMoved(false)
	{
	}


	void Construct(const FArguments& InArgs, FString InTitle, float WindowPosX, float WindowPosY, float WindowWidth,
					float WindowHeight);

	void NotifyWindowClosed(const TSharedRef<SWindow>& ClosedWindow);

	void NotifyWindowMoved(const TSharedRef<SWindow>& MovedWindow);


	/**
	 * Event notification for the 'SUSPEND/RESUME' button
	 *
	 * @return	Returns whether the event was handled or not
	 */
	FReply NotifySuspendClicked();

public:
	/** The log widget, which is the primary widget contained in this window */
	TSharedPtr<SLogWidget> LogWidget;

	/** Multicast OnWindowClosed notifications */
	FMultiOnWindowClosed MultiOnWindowClosed;

	/** Whether or not the window has moved, since creation (used to detect windows coming out of minimize state) */
	bool bHasMoved;
};
