// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Layout/Geometry.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Widgets/Views/STableRow.h"
#include "NetcodeUnitTest.h"


/**
 * SMultiSelectTableRow
 *
 * Implements a SListView row, that supports selection of multiple rows, using just the mouse
 */
template <typename ItemType>
class SMultiSelectTableRow : public STableRow<ItemType>
{
public:
	SMultiSelectTableRow()
		: STableRow<ItemType>()
	{
	}

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = STableRow<ItemType>::OnDragDetected(MyGeometry, MouseEvent);

		auto OwnerWidget = this->OwnerTablePtr.Pin();

		// When the user starts dragging a log line, treat that as the start of selecting multiple-lines
		if (this->GetSelectionMode() == ESelectionMode::Multi)
		{
			const ItemType* MyItem = OwnerWidget->Private_ItemFromWidget(this);

			// Unless 'shift' is being held to expand selection, reset selection to the current item
			if (!MouseEvent.IsShiftDown() || !OwnerWidget->Private_IsItemSelected(*MyItem))
			{
				OwnerWidget->Private_SetItemSelection(*MyItem, true, true);
				OwnerWidget->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
			}

			Reply = FReply::Handled().BeginDragDrop(MakeShareable(new FDragDropOperation()));
		}

		return Reply;
	}

	virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override
	{
		STableRow<ItemType>::OnDragEnter(MyGeometry, DragDropEvent);

		auto OwnerWidget = this->OwnerTablePtr.Pin();

		// Each time the user mouses-over another log line when dragging, select all lines from the dragged line to the current line
		if (this->GetSelectionMode() == ESelectionMode::Multi && OwnerWidget->Private_GetNumSelectedItems() > 0)
		{
			const ItemType* MyItem = OwnerWidget->Private_ItemFromWidget(this);

			OwnerWidget->Private_ClearSelection();
			OwnerWidget->Private_SelectRangeFromCurrentTo(*MyItem);
		}
	}

};
