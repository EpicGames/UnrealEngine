// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Misc/EnumClassFlags.h"
#include "Styling/SlateColor.h"


// Forward declarations
class UNetConnection;
class FUnitLogInterface;
class UUnitTest;


/**
 * Enums
 */

// @todo #JohnBLowPri: Status window logs (such as unit test success/failure), don't add the same colouring to main game log window;
//				add this sometime (probably based on how FUnitTestProcess does it)


// @todo #JohnBRefactor: Trim values from the below enum - not all are necessary

/**
 * Used to help identify what type of log is being processed
 *
 * NOTE: This enum is not compatible with UENUM (that requires uint8)
 */
enum class ELogType : uint32
{
	None				= 0x00000000,					// Not set

	/** What part of the engine the log message originates from locally (may not be set) */
	OriginUnitTest		= 0x00000001,					// Log originating from unit test code
	OriginEngine		= 0x00000002,					// Log originating from an engine event in the unit test (e.g. unit test Tick)
	OriginNet			= 0x00000004,					// Log originating from netcode for current unit test (specifically net receive)
	OriginConsole		= 0x00000008,					// Log originating from a console command (typically from the unit test window)
	OriginVoid			= 0x00000010,					// Log which should have no origin, and which should be ignored by log capturing

	OriginMask			= OriginUnitTest | OriginEngine | OriginNet | OriginConsole | OriginVoid,


	/** What class of unit test log this is */
	Local				= 0x00000080,					// Log from locally executed code (displayed in 'Local' tab)
	Server				= 0x00000100,					// Log from a server instance (displayed in 'Server' tab)
	Client				= 0x00000200,					// Log from a client instance (displayed in 'Client' tab)


	/** What class of unit test status log this is (UNIT_LOG vs UNIT_STATUS_LOG/STATUS_LOG) */
	GlobalStatus		= 0x00000400,					// Status placed within the overall status window
	UnitStatus			= 0x00000800,					// Status placed within the unit test window

	/** Status log modifiers (used with the UNIT_LOG and UNIT_STATUS_LOG/STATUS_LOG macro's) */
	StatusImportant		= 0x00001000,					// An important status event (displayed in the 'Summary' tab)
	StatusSuccess		= 0x00002000 | StatusImportant,	// Success event status
	StatusWarning		= 0x00004000 | StatusImportant,	// Warning event status
	StatusFailure		= 0x00008000 | StatusImportant,	// Failure event status
	StatusError			= 0x00010000 | StatusFailure,	// Error/Failure event status, that triggers an overall unit test failure
	StatusDebug			= 0x00020000,					// Debug status (displayed in the 'Debug' tab)
	StatusAdvanced		= 0x00040000,					// Status event containing advanced/technical information
	StatusVerbose		= 0x00080000,					// Status event containing verbose information
	StatusAutomation	= 0x00100000,					// Status event which should be printed out to the automation tool


	/** Log output style modifiers */
	StyleBold			= 0x00200000,					// Output text in bold
	StyleItalic			= 0x00400000,					// Output text in italic
	StyleUnderline		= 0x00800000,					// Output pseudo-underline text (add newline and --- chars; UE can't underline)
	StyleMonospace		= 0x01000000,					// Output monospaced text (e.g. for list tab formatting); can't use bold/italic

	All					= 0xFFFFFFFF,


	/** Log lines that should request focus when logged (i.e. if not displayed in the current tab, switch to a tab that does display) */
	FocusMask			= OriginConsole,
};

ENUM_CLASS_FLAGS(ELogType);


/**
 * Globals
 */

// IMPORTANT: If you add more engine-log-capture globals, you must add them to the 'UNIT_EVENT_CLEAR' and related macros

/** Used by to aid in hooking log messages triggered by unit tests */
extern NETCODEUNITTEST_API FUnitLogInterface*	GActiveLogInterface;

/** The current ELogType flag modifiers, associated with a UNIT_LOG or UNIT_STATUS_LOG/STATUS_LOG call */
extern NETCODEUNITTEST_API ELogType				GActiveLogTypeFlags;

// @todo #JohnB: See if this needs updating
// @todo #JohnB: Document
extern FUnitLogInterface*						GActiveLogEngineEvent;
extern UWorld*									GActiveLogWorld; // @todo #JohnBReview: Does this make other log hooks redundant?

/** Keeps track of the UUnitTestNetConnection, currently processing received data (to aid with selective log hooking) */
extern UNetConnection*							GActiveReceiveUnitConnection;



/**
 * Macro defines
 */

/**
 * Helper function, for macro defines - allows the 'UnitLogTypeFlags' macro parameters below, to be optional
 */
inline ELogType OptionalFlags(ELogType InFlags=ELogType::None)
{
	return InFlags;
}

// Used with the below macros, and sometimes to aid with large unit tests logs

#define UNIT_LOG_BEGIN(UnitLogObj, UnitLogTypeFlags) \
	{ \
		FUnitLogInterface* OldUnitLogVal = GActiveLogInterface; \
		GActiveLogInterface = (UnitLogObj != nullptr ? UnitLogObj : GActiveLogInterface); \
		GActiveLogTypeFlags = ELogType::UnitStatus | OptionalFlags(UnitLogTypeFlags);

#define UNIT_LOG_END() \
		GActiveLogTypeFlags = ELogType::None; \
		GActiveLogInterface = OldUnitLogVal; \
	}

#define UNIT_LOG_VOID_START() \
	GActiveLogTypeFlags = ELogType::OriginVoid;

#define UNIT_LOG_VOID_END() \
	GActiveLogTypeFlags = ELogType::None;


/**
 * Special log macro for unit tests, to aid in hooking logs from these unit tests
 * NOTE: These logs are scoped
 *
 * @param UnitLogObj		Reference to the unit test this log message is originating from
 * @param UnitLogTypeFlags	(Optional) The additional ELogFlags flags, to be applied to this log
 * @param Format			The format of the log message
 */
#define UNIT_LOG_OBJ(UnitLogObj, UnitLogTypeFlags, Format, ...) \
	UNIT_LOG_BEGIN(UnitLogObj, UnitLogTypeFlags); \
		if ((OptionalFlags(UnitLogTypeFlags) & ELogType::StatusError) == ELogType::StatusError) \
		{ \
			UE_LOG(LogUnitTest, Error, Format, ##__VA_ARGS__); \
		} \
		else if ((OptionalFlags(UnitLogTypeFlags) & ELogType::StatusWarning) == ELogType::StatusWarning) \
		{ \
			UE_LOG(LogUnitTest, Warning, Format, ##__VA_ARGS__); \
		} \
		else \
		{ \
			UE_LOG(LogUnitTest, Log, Format, ##__VA_ARGS__); \
		} \
	UNIT_LOG_END(); \
	UNIT_LOG_VOID_START(); \
	if ((OptionalFlags(UnitLogTypeFlags) & ELogType::StatusError) == ELogType::StatusError) \
	{ \
		UUnitTest* UnitTestRef = UnitLogObj != nullptr ? UnitLogObj->GetUnitTest() : nullptr; \
		UE_LOGF(LogUnitTest, Error, "%ls: %ls", (UnitTestRef != nullptr ? *(UnitTestRef->GetUnitTestName()) : TEXT("nullptr")), \
				*FString::Printf(Format, ##__VA_ARGS__)); \
	} \
	else if ((OptionalFlags(UnitLogTypeFlags) & ELogType::StatusWarning) == ELogType::StatusWarning) \
	{ \
		UUnitTest* UnitTestRef = UnitLogObj != nullptr ? UnitLogObj->GetUnitTest() : nullptr; \
		UE_LOGF(LogUnitTest, Warning, "%ls: %ls", (UnitTestRef != nullptr ? *(UnitTestRef->GetUnitTestName()) : TEXT("nullptr")), \
				*FString::Printf(Format, ##__VA_ARGS__)); \
	} \
	UNIT_LOG_VOID_END();


// More-concise version of the above macro

#define UNIT_LOG(UnitLogTypeFlags, Format, ...) \
	UNIT_LOG_OBJ(this, UnitLogTypeFlags, Format, ##__VA_ARGS__)


/**
 * Special log macro, for messages that should be printed to the unit test status window
 */
#define STATUS_LOG_BASE(UnitLogTypeFlags, Format, ...) \
	{ \
		GActiveLogTypeFlags = ELogType::GlobalStatus | OptionalFlags(UnitLogTypeFlags); \
		UUnitTestManager* Manager = UUnitTestManager::Get(); \
		Manager->SetStatusLog(true); \
		Manager->Logf(Format, ##__VA_ARGS__); \
		Manager->SetStatusLog(false); \
		\
		if ((GActiveLogTypeFlags & ELogType::StatusAutomation) == ELogType::StatusAutomation && GIsAutomationTesting) \
		{ \
			GWarn->Logf(Format, ##__VA_ARGS__); \
		} \
		\
		GActiveLogTypeFlags = ELogType::None; \
	}

#define STATUS_LOG_OBJ(UnitLogObj, UnitLogTypeFlags, Format, ...) \
	{ \
		auto StatusLogWrap = [&](FUnitLogInterface* InUnitLogObj) \
			{ \
				GActiveLogTypeFlags = ELogType::GlobalStatus | OptionalFlags(UnitLogTypeFlags); \
				UUnitTest* SourceUnitTest = InUnitLogObj != nullptr ? InUnitLogObj->GetUnitTest() : nullptr; \
				UNIT_EVENT_CLEAR; \
				if ((GActiveLogTypeFlags & ELogType::StatusError) == ELogType::StatusError) \
				{ \
					if (SourceUnitTest != nullptr) \
					{ \
						UE_LOGF(LogUnitTest, Error, "%ls: %ls", *SourceUnitTest->GetUnitTestName(), \
								*FString::Printf(Format, ##__VA_ARGS__)); \
					} \
					else \
					{ \
						UE_LOG(LogUnitTest, Error, Format, ##__VA_ARGS__) \
					} \
				} \
				else if ((GActiveLogTypeFlags & ELogType::StatusWarning) == ELogType::StatusWarning) \
				{ \
					if (SourceUnitTest != nullptr) \
					{ \
						UE_LOGF(LogUnitTest, Warning, "%ls: %ls", *SourceUnitTest->GetUnitTestName(), \
								*FString::Printf(Format, ##__VA_ARGS__)); \
					} \
					else \
					{ \
						UE_LOG(LogUnitTest, Warning, Format, ##__VA_ARGS__) \
					} \
				} \
				else \
				{ \
					UE_LOG(LogUnitTest, Log, Format, ##__VA_ARGS__) \
				} \
				UNIT_EVENT_RESTORE; \
				STATUS_LOG_BASE(UnitLogTypeFlags, Format, ##__VA_ARGS__) \
				GActiveLogTypeFlags = ELogType::None; \
			}; \
		\
		StatusLogWrap(UnitLogObj); \
	}

#define STATUS_LOG(UnitLogTypeFlags, Format, ...) STATUS_LOG_OBJ(nullptr, UnitLogTypeFlags, Format, ##__VA_ARGS__)

/**
 * Version of the above, for unit test status window entries, which are from specific unit tests
 */
#define UNIT_STATUS_LOG_OBJ(UnitLogObj, UnitLogTypeFlags, Format, ...) \
	UNIT_LOG_BEGIN(UnitLogObj, UnitLogTypeFlags); \
	STATUS_LOG_OBJ(UnitLogObj, UnitLogTypeFlags, Format, ##__VA_ARGS__) \
	UNIT_LOG_END();

#define UNIT_STATUS_LOG(UnitLogTypeFlags, Format, ...) \
	UNIT_STATUS_LOG_OBJ(this, UnitLogTypeFlags, Format, ##__VA_ARGS__)


/**
 * Changes/resets the colour of messages printed to the unit test status window
 */
#define STATUS_SET_COLOR(InColor) \
	{ \
		UUnitTestManager* Manager = UUnitTestManager::Get(); \
		Manager->SetStatusColor(InColor); \
	}

#define STATUS_RESET_COLOR() \
	{ \
		UUnitTestManager* Manager = UUnitTestManager::Get(); \
		Manager->SetStatusColor(); /* No value specified = reset */ \
	}


// @todo #JohnBDoc: Document these macros

#define UNIT_EVENT_BEGIN(UnitLogObj) \
	FUnitLogInterface* OldEventLogVal = GActiveLogEngineEvent; \
	GActiveLogEngineEvent = UnitLogObj;
	
#define UNIT_EVENT_END \
	GActiveLogEngineEvent = OldEventLogVal;

/**
 * Stores and then clears all engine-log-capture events (in order to prevent capture of a new log entry)
 * NOTE: Do not clear GActiveLogUnitTest here, as some macros that use this, rely upon that
 */
#define UNIT_EVENT_CLEAR \
	FUnitLogInterface* OldEventLogVal = GActiveLogEngineEvent; \
	UWorld* OldEventLogWorld = GActiveLogWorld; \
	GActiveLogEngineEvent = nullptr; \
	GActiveLogWorld = nullptr;

/**
 * Restores all stored/cleared engine-log-capture events
 */
#define UNIT_EVENT_RESTORE \
	UNIT_EVENT_END \
	GActiveLogWorld = OldEventLogWorld;


/**
 * Implements an interface for using the UNIT_LOG etc. macros above, within a class
 */
class NETCODEUNITTEST_API FUnitLogInterface
{
public:
	/**
	 * Base constructor
	 */
	FUnitLogInterface()
		: LogColor(FSlateColor::UseForeground())
	{
	}

	virtual ~FUnitLogInterface()
	{
	}


	/**
	 * Whether or not this log interface is the source of a UNetConnection - used for deciding where to direct, UNetConnection log hooks
	 *
	 * @param InConnection	The connection to check
	 * @return				Whether or not the connection is associated with this log interface
	 */
	virtual bool IsConnectionLogSource(UNetConnection* InConnection)
	{
		return false;
	}

	/**
	 * Whether or not this log interface is the source of an FTimerManager delegate object,
	 * used for hooking logs triggered during global timer manager delegates.
	 *
	 * @param InTimerDelegateObject		The UObject that a timer delegate is tied to
	 * @return							Whether or not the object is associated with this log interface
	 */
	virtual bool IsTimerLogSource(UObject* InTimerDelegateObject)
	{
		return false;
	}

	/**
	 * For implementation in subclasses, for helping to track local log entries related to a unit test.
	 * This covers logs from within the unit test, and (for UClientUnitTest) from processing net packets related to a unit test.
	 *
	 * NOTE: The parameters are the same as the unprocessed log 'serialize' parameters, to convert to a string, use:
	 * FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Data, GPrintLogTimes)
	 *
	 * NOTE: Verbosity ELogVerbosity::SetColor is a special category, whose log messages can be ignored.
	 *
	 * @param InLogType		The type of local log message this is
	 * @param Data			The base log message being written
	 * @param Verbosity		The warning/filter level for the log message
	 * @param Category		The log message category (LogNet, LogNetTraffic etc.)
	 */
	virtual void NotifyLocalLog(ELogType InLogType, const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
	{
	}

	/**
	 * As above, except for log entries directed towards the Unit Test Status window.
	 *
	 * @param InLogType		The type of local log message this is
	 * @param Data			The base log message being written
	 */
	virtual void NotifyStatusLog(ELogType InLogType, const TCHAR* Data)
	{
	}


	/**
	 * Override the default log color
	 *
	 * @param InLogColor	The new log color to use
	 */
	virtual void SetLogColor(FSlateColor InLogColor)
	{
		LogColor = InLogColor;
	}

	/**
	 * Resets the log color to default
	 */
	FORCEINLINE void ClearLogColor()
	{
		SetLogColor(FSlateColor::UseForeground());
	}


	/**
	 * If the log interface is tied to a unit test, this matches it up to an active unit test
	 *
	 * @return	Returns the unit test pointer, or nullptr if this isn't a unit test
	 */
	virtual UUnitTest* GetUnitTest();

protected:
	// @todo #JohnB: Convert to pointer, to eliminate extra package dependency.
	/** Overrides the colour of UNIT_LOG log messages */
	FSlateColor LogColor;
};


/**
 * Implements a UNIT_LOG interface, which redirects to another unit log interface (e.g. redirect MinimalClient UNIT_LOG's to a UnitTest)
 */
class FUnitLogRedirect : public FUnitLogInterface
{
public:
	/**
	 * Base constructor
	 */
	FUnitLogRedirect()
		: FUnitLogInterface()
		, TargetInterface(nullptr)
	{
	}

	/**
	 * Sets the interface we should redirect to
	 *
	 * @param InInterface	The interface to redirect to
	 */
	void SetInterface(FUnitLogInterface* InInterface)
	{
		TargetInterface = InInterface;
	}


	// Do NOT pass this back to TargetInterface, because if the TargetInterface is the source, we can't be the source
	// (would cause an infinite loop, for UnitTask->ClientUnitTest->UnitTask)
#if 0
	virtual bool IsConnectionSource(UNetConnection* InConnection) override
	{
	}
#endif

	virtual void NotifyLocalLog(ELogType InLogType, const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
		override
	{
		if (TargetInterface != nullptr)
		{
			TargetInterface->NotifyLocalLog(InLogType, Data, Verbosity, Category);
		}
	}

	virtual void NotifyStatusLog(ELogType InLogType, const TCHAR* Data) override
	{
		if (TargetInterface != nullptr)
		{
			TargetInterface->NotifyStatusLog(InLogType, Data);
		}
	}

	virtual UUnitTest* GetUnitTest() override
	{
		return (TargetInterface != nullptr ? TargetInterface->GetUnitTest() : nullptr);
	}

	virtual void SetLogColor(FSlateColor InLogColor)
	{
		if (TargetInterface != nullptr)
		{
			TargetInterface->SetLogColor(InLogColor);
		}
	}

private:
	/** The interface to redirect to */
	FUnitLogInterface* TargetInterface;
};
