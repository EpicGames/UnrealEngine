// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "NUTEnum.h"


// Forward declarations
class APlayerController;
class FUnitTestEnvironment;
enum class EUnitTestFlags : uint32;
class UUnitTest;


// Globals

/** The list of registered unit test environments, mapped to game name (unit test modules should add to this, within StartupModule) */
extern TMap<FString, FUnitTestEnvironment*> UnitTestEnvironments;


/**
 * Class for handling per-game implementation of unit test environmental defaults
 */
class NETCODEUNITTEST_API FUnitTestEnvironment
{
	friend class FNetcodeUnitTest;
	friend class UUnitTestManager;
	friend class UClientUnitTest;

public:
	FUnitTestEnvironment()
		: UnitTest(nullptr)
	{
	}

	/**
	 * Registers the unit test environment, with the environment list
	 * IMPORTANT: This MUST be implemented in every subclass, and the subclass version MUST called, within your modules 'StartupModule'
	 */
	static void Register()
	{
		// A special unit test environment, for unit tests which support all games
		AddUnitTestEnvironment(TEXT("NullUnitEnv"), new FUnitTestEnvironment());
	}

	/**
	 * Adds a new environment to the unit test environment list
	 *
	 * @param Game	The game the unit test environment is for
	 * @param Env	The environment being added
	 */
	static void AddUnitTestEnvironment(FString Game, FUnitTestEnvironment* Env);

	virtual ~FUnitTestEnvironment()
	{
	}

private:
	/**
	 * Winds-down the UnitTestEnvironments map; does not need implementation in subclasses
	 */
	static void Unregister()
	{
		TArray<FUnitTestEnvironment*> EnvList;

		UnitTestEnvironments.GenerateValueArray(EnvList);

		for (auto CurEntry : EnvList)
		{
			if (CurEntry != NULL)
			{
				delete CurEntry;
			}
		}

		UnitTestEnvironments.Empty();
	}

public:
	/**
	 * Returns the default/minimum timeout that unit tests should use, for the current game
	 *
	 * @return	The default unit test timeout for the current game
	 */
	virtual uint32 GetDefaultUnitTestTimeout()
	{
		return 60;
	}

	/**
	 * Returns the default map name, that should be used with unit tests, for the current game
	 *
	 * @param UnitTestFlags		The flags for the current unit test - affects chosen map in some games
	 * @return					Returns the default map name
	 */
	virtual FString GetDefaultMap(EUnitTestFlags UnitTestFlags);

	/**
	 * Returns the default parameters for launched servers
	 *
	 * @param InLogCmds		Used to pass in -LogCmds= entries, which need special parsing (don't specify the -LogCmds= part)
	 * @param InExecCmds	Used to pass in -ExecCmds= entries, which need special parsing (don't specify the -ExecCmds= part)
	 * @return				Returns the default parameters
	 */
	FString GetDefaultServerParameters(FString InLogCmds=TEXT(""), FString InExecCmds=TEXT(""));

	/**
	 * Sets up the default server parameters, specifying commandline and '-*Cmds=x' separately (combined within parent function above)
	 *
	 * @param OutCommandline	Passes in the current state of the commandline, for modification
	 * @param OutLogCmds		Passes in the current LogCmds list, for modification
	 * @param OutExecCmds		Passes in the current ExecCmds list, for modification
	 */
	virtual void SetupDefaultServerParameters(FString& OutCommandline, FString& OutLogCmds, FString& OutExecCmds)
	{
	}

	/**
	 * Returns the default parameters for launched clients
	 *
	 * @return	Returns the default parameters
	 */
	FString GetDefaultClientParameters();

	/**
	 * Sets up the default client parameters, specifying a (potentially already modified) commandline for modification
	 *
	 * @param OutCommandline	Passes in the current state of the commandline, for modification
	 */
	virtual void SetupDefaultClientParameters(FString& OutCommandline)
	{
	}

	/**
	 * Returns the default URL that minimal clients should use, when connecting to a server, for the current game
	 *
	 * @return	Returns the default URL
	 */
	virtual FString GetDefaultClientConnectURL();

	/**
	 * Returns server log messages, for the current game, that indicate progress in starting up
	 *
	 * @param OutStartProgressLogs	Outputs partial server log messages that indicate progress starting up
	 * @param OutReadyLogs			Outputs partial server log messages that indicate startup is complete
	 * @param OutTimeoutResetLogs	Outputs partial server log messages that indicate resetting unit-test/connection timeout is required
	 */
	void GetServerProgressLogs(const TArray<FString>*& OutStartProgressLogs, const TArray<FString>*& OutReadyLogs,
										const TArray<FString>*& OutTimeoutResetLogs);

	/**
	 * Called when initializing the static arrays, containing the server progress logs
	 *
	 * @param StartProgressLogs		The static array containing logs indicating progress starting up
	 * @param ReadyLogs				The static array containing logs indicating startup is complete
	 * @param TimeoutResetLogs		The static array containing logs indicating resetting of unit-test/connection timeout is required
	 */
	virtual void InitializeServerProgressLogs(TArray<FString>& StartProgressLogs, TArray<FString>& ReadyLogs,
												TArray<FString>& TimeoutResetLogs)
	{
	}


	/**
	 * Returns client log messages, for the current game, that indicate progress logs for resetting unit test timeout
	 *
	 * @param OutTimeoutResetLogs	Outputs partial client log messages that indicate resetting unit-test/connection timeout is required
	 */
	void GetClientProgressLogs(const TArray<FString>*& OutTimeoutResetLogs);

	/**
	 * Called when initializing the static arrays, containing the client progress logs
	 *
	 * @param TimeoutResetLogs		The static array containing logs indicating resetting of unit-test/connection timeout is required
	 */
	virtual void InitializeClientProgressLogs(TArray<FString>& TimeoutResetLogs)
	{
	}


	/**
	 * Returns child process names, that indicate progress blockers in starting up (e.g. shader compiler)
	 * All child processes, including server/client child processes (and their children) are checked.
	 *
	 * @param OutBlockingProcesses	Outputs process names, for processes which indicate a blocking task, which is blocking progression
	 */
	void GetProgressBlockingProcesses(const TArray<FString>*& OutBlockingProcesses);

	/**
	 * Called when initializing the static arrays, containing the progress blocking processes
	 *
	 * @param BlockingProcesses		The static array containing processes indicating a blocking task
	 */
	virtual void InitializeProgressBlockingProcesses(TArray<FString>& BlockingProcesses)
	{
	}


	/**
	 * Executes game-specific handling, upon client unit tests initializing the client PlayerController
	 *
	 * @param UnitTestFlags	The flags for the current unit test - affects handling, depending upon game
	 * @param PC			The PlayerController being initialized
	 */
	virtual void HandleClientPlayer(EUnitTestFlags UnitTestFlags, APlayerController* PC)
	{
	}

	/**
	 * If the game environment requires special/complex setup (e.g. player auth before join),
	 * then UnitTask's for handling complex setup, are added here.
	 */
	virtual void InitializeUnitTasks()
	{
	}


protected:
	// @todo #JohnB: Replace references to UnitTestFlags in above functions, with usage of the UnitTest value below, instead

	/** The unit test being initialized using the current environment (WARNING: Will be nullptr for legacy unit tests) */
	UUnitTest* UnitTest;
};
